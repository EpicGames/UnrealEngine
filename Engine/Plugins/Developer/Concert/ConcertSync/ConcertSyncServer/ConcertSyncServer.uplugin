{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Concert Sync - Server",
	"Description": "Server plugin to enables multi-users editor sessions",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor" ],
	"Modules": [
		{
			"Name": "ConcertSyncServer",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default",
			"ProgramAllowList": [ "UnrealMultiUserServer", "CoopMultiUserServer", "UnrealMultiUserSlateServer", "UnrealRecoverySvc", "CrashReportClientEditor" ]
		}
	],
	"Plugins": [
		{
			"Name": "ConcertMain",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		}
	]
}
