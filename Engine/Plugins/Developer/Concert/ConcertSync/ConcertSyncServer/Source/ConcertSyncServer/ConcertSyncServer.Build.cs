// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ConcertSyncServer : ModuleRules
	{
		public ConcertSyncServer(ReadOnlyTargetRules Target) : base(Target)
		{
			// Make the linking path shorter (to prevent busting the Windows limit) when linking ConcertSyncServer.lib against an executable that have a long name.
			ShortName = "CncrtSyncSvr";

			// ConcertSyncServerLoop.inl depends on LaunchEngineLoop.h, but is never included in this module.
			IWYUSupport = IWYUSupport.None;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Concert",
					"ConcertSyncCore",
					"ConcertServer"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ConcertTransport",
					"Serialization",
					"JsonUtilities"
				}
			);
		}
	}
}
