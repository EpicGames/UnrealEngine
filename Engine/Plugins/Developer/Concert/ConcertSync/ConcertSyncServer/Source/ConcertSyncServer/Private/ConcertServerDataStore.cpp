// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerDataStore.h"
#include "ConcertDataStoreMessages.h"
#include "IConcertSession.h"
#include "IConcertSessionHandler.h"
#include "ConcertSyncServerLiveSession.h"
#include "ConcertLogGlobal.h"

FConcertServerDataStore::FConcertServerDataStore(TSharedRef<FConcertSyncServerLiveSession> InLiveSession, bool bIsContentReplicationEnabled)
	: DataStore(FConcertDataStore::EUpdatePolicy::Overwrite)
	, LiveSession(MoveTemp(InLiveSession))
	, bContentReplicationEnabled(bIsContentReplicationEnabled)
{
	check(LiveSession->IsValidSession());

	// Register the request handlers into the provider.
	LiveSession->GetSession().RegisterCustomRequestHandler<FConcertDataStore_FetchOrAddRequest, FConcertDataStore_Response, FConcertServerDataStore>(this, &FConcertServerDataStore::OnFetchOrAdd);
	LiveSession->GetSession().RegisterCustomRequestHandler<FConcertDataStore_CompareExchangeRequest, FConcertDataStore_Response, FConcertServerDataStore>(this, &FConcertServerDataStore::OnCompareExchange);

	if (bContentReplicationEnabled)
	{
		LiveSession->GetSession().OnSessionClientChanged().AddRaw(this, &FConcertServerDataStore::OnSessionClientChanged);
	}
}

FConcertServerDataStore::~FConcertServerDataStore()
{
	// Unregister the request handlers from the provider.
	LiveSession->GetSession().UnregisterCustomRequestHandler<FConcertDataStore_FetchOrAddRequest>();
	LiveSession->GetSession().UnregisterCustomRequestHandler<FConcertDataStore_CompareExchangeRequest>();
}

EConcertSessionResponseCode FConcertServerDataStore::OnFetchOrAdd(const FConcertSessionContext& Context, const FConcertDataStore_FetchOrAddRequest& Request, FConcertDataStore_Response& Response)
{
	// Call the store.
	FConcertDataStoreResult Result = DataStore.FetchOrAdd(Request.Key, Request.TypeName, Request.SerializedValue);

	// Setup the response. Sent back a value only if it was fetched.
	Response.ResultCode = Result.Code;
	if (Result.Code == EConcertDataStoreResultCode::Fetched)
	{
		Response.Value = *Result.Value;
	}
	else if (Result.Code == EConcertDataStoreResultCode::Added)
	{
		// Notify the other clients (if any).
		FireContentReplicationEvent(Context.SourceEndpointId, Request.Key, *Result.Value);
	}

	// At the protocol level, the request succeeded. The function was successfully called whatever the function returned.
	return EConcertSessionResponseCode::Success;
}

EConcertSessionResponseCode FConcertServerDataStore::OnCompareExchange(const FConcertSessionContext& Context, const FConcertDataStore_CompareExchangeRequest& Request, FConcertDataStore_Response& Response)
{
	SCOPED_CONCERT_TRACE(FConcertServerDataStore_OnCompareExchange);
	// Ensure we have a version or an expected value, but not both.
	check((Request.ExpectedVersion != 0 && Request.Expected.PayloadSize == 0) || (Request.ExpectedVersion == 0 && Request.Expected.PayloadSize != 0));

	// Try to fetch the value.
	FConcertDataStoreResult Result = DataStore.Fetch(Request.Key, Request.TypeName);

	// If the value exist.
	if (Result.Code == EConcertDataStoreResultCode::Fetched)
	{
		// If the version or value matches.
		if ((Request.ExpectedVersion != 0 && Request.ExpectedVersion == Result.Value->Version) ||
		    (Request.ExpectedVersion == 0
			 && Result.Value->SerializedValue.PayloadCompressionMethod == Request.Expected.PayloadCompressionMethod
			 && Result.Value->SerializedValue.PayloadCompressionBias == Request.Expected.PayloadCompressionBias
			 && Result.Value->SerializedValue.PayloadSize == Request.Expected.PayloadSize
			 && Result.Value->SerializedValue.PayloadBytes.Bytes == Request.Expected.PayloadBytes.Bytes))
		{
			// Replace the current value with the desired one.
			Result = DataStore.Store(Request.Key, Result.Value->TypeName, Request.Desired);
			check(Result.Code == EConcertDataStoreResultCode::Exchanged && Result.Value);

			// Send the version back because the client may have successfully exchanged a version that was good few versions before,
			// for example, client sent '3' at version '2', but the value has changed few times and now is '3' at version '7'. The
			// exchange will succeed, but the exact version is '8', not '4'. Don't send back the "stored" value nor the "type name" because
			// the user is expected to know them since they sent them.
			Response.Value.Version = Result.Value->Version;

			// Notify the other clients (if any).
			FireContentReplicationEvent(Context.SourceEndpointId, Request.Key, *Result.Value);
		}
		else
		{
			check(Result.Code == EConcertDataStoreResultCode::Fetched && Result.Value);
			Response.Value = *Result.Value;
		}

		// Set up the response code.
		Response.ResultCode = Result.Code;
	}
	else
	{
		// The value was not found or types did not match.
		check(Result.Code == EConcertDataStoreResultCode::NotFound || Result.Code == EConcertDataStoreResultCode::TypeMismatch);
		Response.ResultCode = Result.Code;
	}

	// At the protocol level, the request succeeded. The function was successfully called whatever the function returned.
	return EConcertSessionResponseCode::Success;
}

void FConcertServerDataStore::OnSessionClientChanged(IConcertServerSession& InSession, EConcertClientStatus Status, const FConcertSessionClientInfo& ClientInfo)
{
	check(&LiveSession->GetSession() == &InSession);

	if (Status == EConcertClientStatus::Connected && DataStore.GetSize() > 0)
	{
		// Build up the list of stored values into a replication event.
		FConcertDataStore_ReplicateEvent ReplicateEvent;
		DataStore.Visit([&ReplicateEvent](const FName& Key, const FConcertDataStore_StoreValue& Value)
		{
			ReplicateEvent.Values.Add(FConcertDataStore_KeyValuePair{Key, Value});
		});

		// Replicate the current store content to the client.
		LiveSession->GetSession().SendCustomEvent(ReplicateEvent, ClientInfo.ClientEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
}

void FConcertServerDataStore::FireContentReplicationEvent(const FGuid& InstigatorEndpointId, const FName& Key, const FConcertDataStore_StoreValue& Value)
{
	if (bContentReplicationEnabled)
	{
		FConcertDataStore_ReplicateEvent ReplicateChangeEvent;
		ReplicateChangeEvent.Values.Add(FConcertDataStore_KeyValuePair{Key, Value});

		// Push the updates to all clients, except the one who performed the change.
		TArray<FGuid> ClientEndpointIds = LiveSession->GetSession().GetSessionClientEndpointIds();
		ClientEndpointIds.Remove(InstigatorEndpointId);
		LiveSession->GetSession().SendCustomEvent(ReplicateChangeEvent, ClientEndpointIds, EConcertMessageFlags::ReliableOrdered);
	}
}

namespace UE::ConcertSyncTests::DatastoreTests
{

// This function is private and declared "extern" in the ConcertSyncTest module. It enables creating a server data store to test the client/server
// integration without including any header files. This prevent leaking the ConcertServerDataStore.h in the module public includes since this is not
// required for general usage. The function returns an opaque pointer because the class doesn't have any useful API, all the interaction is done
// through the custom request/response/event protocol provided by the Session.
CONCERTSYNCSERVER_API TSharedPtr<void> MakeConcerteServerDataStoreForTest(TSharedRef<IConcertServerSession> Session, bool bEnableContentReplication)
{
	return MakeShared<FConcertServerDataStore>(MakeShared<FConcertSyncServerLiveSession>(MoveTemp(Session), EConcertSyncSessionFlags::None), bEnableContentReplication);
}

} // namespace ConcertDataStoreTestUtils
