// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"
#include "ConcertMessages.h"
#include "ConcertDataStore.h"
#include "ConcertDataStoreMessages.h"

class IConcertServerSession;
class FConcertSyncServerLiveSession;
struct FConcertSessionContext;

/**
 * Handles data store requests from one or more clients connected to the same
 * Concert session. The store hooks itself up as a request handler in the specified
 * session.
 *
 * @remarks The implementation is not thread safe. The concert server application
 *          is expected to pump and dispatch session requests serially to the
 *          data store.
 */
class FConcertServerDataStore
{
public:
	/**
	 * Constructs a server-side data store.
	 *
	 * @param LiveSession
	 *     The session owning the data store.
	 *
	 * @param bIsContentReplicationEnabled
	 *     If true, the servers will pushes the entire store content to newly connected clients
	 *     and continue updating each of them as keys get added or updated. In normal utilization
	 *     this is expected to be true, but for unit tests, it is useful to be able to turn this
	 *     feature off.
	 */
	FConcertServerDataStore(TSharedRef<FConcertSyncServerLiveSession> LiveSession, bool bIsContentReplicationEnabled = true);
	virtual ~FConcertServerDataStore();

private:
	void OnSessionClientChanged(IConcertServerSession&, EConcertClientStatus, const FConcertSessionClientInfo&);
	EConcertSessionResponseCode OnFetchOrAdd(const FConcertSessionContext& Context, const FConcertDataStore_FetchOrAddRequest& Request, FConcertDataStore_Response& Response);
	EConcertSessionResponseCode OnCompareExchange(const FConcertSessionContext& Context, const FConcertDataStore_CompareExchangeRequest& Request, FConcertDataStore_Response& Response);
	void FireContentReplicationEvent(const FGuid& SourceEndPoint, const FName& Key, const FConcertDataStore_StoreValue& Value);

	/** Maps the property name to an arbitrary blob value that can be memory compared. */
	FConcertDataStore DataStore;

	/** The concert live session owning the store and through which the requests/responses are dispatched. */
	TSharedPtr<FConcertSyncServerLiveSession> LiveSession;

	/** True if the server should perform initial sync and push further modifications to the clients. */
	bool bContentReplicationEnabled;
};
