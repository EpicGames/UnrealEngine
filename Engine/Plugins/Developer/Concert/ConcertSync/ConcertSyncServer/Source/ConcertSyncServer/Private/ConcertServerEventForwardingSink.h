// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertServerEventSink.h"
#include "ConcertMessageData.h"
#include "ConcertServerEvents.h"

/**
 * Uses CRTP to call IConcertServerEventSink::FooImpl and calls the equivalent event in ConcertServerEvents with the result.
 * Example: TDerived must implement GetSessionsFromPathImpl.
 */
template<typename TDerived>
class TConcertServerEventForwardingSink : public IConcertServerEventSink
{
	TDerived* This() { return static_cast<TDerived*>(this); }
	const TDerived* This() const { return static_cast<TDerived*>(this); }
public:

	//~ Begin IConcertServerEventSink Interface
	virtual void GetSessionsFromPath(const IConcertServer& InServer, const FString& InPath, TArray<FConcertSessionInfo>& OutSessionInfos, TArray<FDateTime>* OutSessionCreationTimes = nullptr) final
	{
		This()->GetSessionsFromPathImpl(InServer, InPath, OutSessionInfos, OutSessionCreationTimes);
	}

	virtual bool OnLiveSessionCreated(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FInternalLiveSessionCreationParams& AdditionalParams) final
	{
		const bool bSuccess = This()->OnLiveSessionCreatedImpl(InServer, InLiveSession, AdditionalParams);
		// AdditionalParams are internal use only - they should never be exposed publicly!
		ConcertServerEvents::OnLiveSessionCreated().Broadcast(bSuccess, InServer, InLiveSession);
		return bSuccess;
	}
	
	virtual void OnLiveSessionDestroyed(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession) final
	{
		This()->OnLiveSessionDestroyedImpl(InServer, InLiveSession);
		ConcertServerEvents::OnLiveSessionDestroyed().Broadcast(InServer, InLiveSession);
	}
	
	virtual bool OnArchivedSessionCreated(const IConcertServer& InServer, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo) final
	{
		const bool bSuccess = This()->OnArchivedSessionCreatedImpl(InServer, InArchivedSessionRoot, InArchivedSessionInfo);
		ConcertServerEvents::OnArchivedSessionCreated().Broadcast(bSuccess, InServer, InArchivedSessionRoot, InArchivedSessionInfo);
		return bSuccess;
	}
	
	virtual void OnArchivedSessionDestroyed(const IConcertServer& InServer, const FGuid& InArchivedSessionId) final
	{
		This()->OnArchivedSessionDestroyedImpl(InServer, InArchivedSessionId);
		ConcertServerEvents::OnArchivedSessionDestroyed().Broadcast(InServer, InArchivedSessionId);
	}
	
	virtual bool ArchiveSession(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo, const FConcertSessionFilter& InSessionFilter) final
	{
		const bool bSuccess = This()->ArchiveSessionImpl(InServer, InLiveSession, InArchivedSessionRoot, InArchivedSessionInfo, InSessionFilter);
		ConcertServerEvents::ArchiveSession_WithSession().Broadcast(bSuccess, InServer, InLiveSession, InArchivedSessionRoot, InArchivedSessionInfo, InSessionFilter);
		return bSuccess;
	}
	
	virtual bool ArchiveSession(const IConcertServer& InServer, const FString& InLiveSessionWorkingDir, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo, const FConcertSessionFilter& InSessionFilter) final
	{
		const bool bSuccess = This()->ArchiveSessionImpl(InServer, InLiveSessionWorkingDir, InArchivedSessionRoot, InArchivedSessionInfo, InSessionFilter);
		ConcertServerEvents::ArchiveSession_WithWorkingDir().Broadcast(bSuccess, InServer, InLiveSessionWorkingDir, InArchivedSessionRoot, InArchivedSessionInfo, InSessionFilter);
		return bSuccess;
	}

	virtual bool CopySession(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FString& NewSessionRoot, const FConcertSessionFilter& InSessionFilter) final
	{
		const bool bSuccess = This()->CopySessionImpl(InServer, InLiveSession, NewSessionRoot, InSessionFilter);
		ConcertServerEvents::CopySession().Broadcast(bSuccess, InServer, InLiveSession, NewSessionRoot, InSessionFilter);
		return bSuccess;
	}

	virtual bool ExportSession(const IConcertServer& InServer, const FGuid& InSessionId, const FString& DestDir, const FConcertSessionFilter& InSessionFilter, bool bAnonymizeData) final
	{
		const bool bSuccess = This()->ExportSessionImpl(InServer, InSessionId, DestDir, InSessionFilter, bAnonymizeData);
		ConcertServerEvents::ExportSession().Broadcast(bSuccess, InServer, InSessionId, DestDir, InSessionFilter, bAnonymizeData);
		return bSuccess;
	}

	virtual bool RestoreSession(const IConcertServer& InServer, const FGuid& InArchivedSessionId, const FString& InLiveSessionRoot, const FConcertSessionInfo& InLiveSessionInfo, const FConcertSessionFilter& InSessionFilter) final
	{
		const bool bSuccess = This()->RestoreSessionImpl(InServer, InArchivedSessionId, InLiveSessionRoot, InLiveSessionInfo, InSessionFilter);
		ConcertServerEvents::RestoreSession().Broadcast(bSuccess, InServer, InArchivedSessionId, InLiveSessionRoot, InLiveSessionInfo, InSessionFilter);
		return bSuccess;
	}

	virtual bool GetUnmutedSessionActivities(const IConcertServer& InServer, const FGuid& SessionId, int64 FromActivityId, int64 ActivityCount, TArray<FConcertSessionSerializedPayload>& OutActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, bool bIncludeDetails) final
	{
		return This()->GetSessionActivitiesImpl(InServer, SessionId, FromActivityId, ActivityCount, OutActivities, OutEndpointClientInfoMap, bIncludeDetails);
	}

	virtual void OnLiveSessionRenamed(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession) final
	{
		This()->OnLiveSessionRenamedImpl(InServer, InLiveSession);
		ConcertServerEvents::OnLiveSessionRenamed().Broadcast(InServer, InLiveSession);
	}
	
	virtual void OnArchivedSessionRenamed(const IConcertServer& InServer, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo) final
	{
		This()->OnArchivedSessionRenamedImpl(InServer, InArchivedSessionRoot, InArchivedSessionInfo);
		ConcertServerEvents::OnArchivedSessionRenamed().Broadcast(InServer, InArchivedSessionRoot, InArchivedSessionInfo);
	}
	//~ End IConcertServerEventSink Interface
};