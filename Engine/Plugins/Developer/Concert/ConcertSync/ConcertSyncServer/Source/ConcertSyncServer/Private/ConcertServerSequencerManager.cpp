// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerSequencerManager.h"
#include "IConcertSession.h"
#include "ConcertSyncServerLiveSession.h"
#include "ConcertWorkspaceMessages.h"
#include "ConcertLogGlobal.h"
#include "Logging/StructuredLog.h"


FConcertServerSequencerManager::FConcertServerSequencerManager(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession)
{
	BindSession(InLiveSession);
}

FConcertServerSequencerManager::~FConcertServerSequencerManager()
{
	UnbindSession();
}

void FConcertServerSequencerManager::BindSession(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession)
{
	check(InLiveSession->IsValidSession());

	UnbindSession();
	LiveSession = InLiveSession;

	LiveSession->GetSession().OnSessionClientChanged().AddRaw(this, &FConcertServerSequencerManager::HandleSessionClientChanged);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertSequencerCloseEvent>(this, &FConcertServerSequencerManager::HandleSequencerCloseEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertSequencerStateEvent>(this, &FConcertServerSequencerManager::HandleSequencerStateEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertSequencerOpenEvent>(this, &FConcertServerSequencerManager::HandleSequencerOpenEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertSequencerTimeAdjustmentEvent>(this, &FConcertServerSequencerManager::HandleSequencerTimeAdjustmentEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertWorkspaceSyncAndFinalizeCompletedEvent>(this, &FConcertServerSequencerManager::HandleWorkspaceSyncAndFinalizeCompletedEvent);

	LiveSession->GetSession().RegisterCustomEventHandler<FConcertSequencerPreloadRequest>(this, &FConcertServerSequencerManager::HandleSequencerPreloadRequestEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertSequencerPreloadAssetStatusMap>(this, &FConcertServerSequencerManager::HandleSequencerPreloadStatusEvent);
}

void FConcertServerSequencerManager::UnbindSession()
{
	if (LiveSession)
	{
		LiveSession->GetSession().OnSessionClientChanged().RemoveAll(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertSequencerCloseEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertSequencerStateEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertSequencerOpenEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertSequencerTimeAdjustmentEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertWorkspaceSyncAndFinalizeCompletedEvent>(this);

		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertSequencerPreloadRequest>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertSequencerPreloadAssetStatusMap>(this);

		LiveSession.Reset();
	}
}

void FConcertServerSequencerManager::HandleSequencerStateEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerStateEvent& InEvent)
{
	// Create or update the Sequencer state 
	FConcertOpenSequencerState& SequencerState = SequencerStates.FindOrAdd(*InEvent.State.SequenceObjectPath);
	SequencerState.ClientEndpointIds.AddUnique(InEventContext.SourceEndpointId);
	SequencerState.State = InEvent.State;

	// Forward the message to the other clients
	TArray<FGuid> ClientIds = LiveSession->GetSession().GetSessionClientEndpointIds();
	ClientIds.Remove(InEventContext.SourceEndpointId);
	LiveSession->GetSession().SendCustomEvent(InEvent, ClientIds, EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
}

void FConcertServerSequencerManager::HandleSequencerOpenEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerOpenEvent& InEvent)
{
	// Create or update the Sequencer state 
	FConcertOpenSequencerState& SequencerState = SequencerStates.FindOrAdd(*InEvent.SequenceObjectPath);
	SequencerState.ClientEndpointIds.AddUnique(InEventContext.SourceEndpointId);
	SequencerState.State.SequenceObjectPath = InEvent.SequenceObjectPath;
	SequencerState.TakeData = InEvent.TakeData.Bytes.Num() > 0 ? InEvent.TakeData : SequencerState.TakeData;

	// Forward the message to the other clients
	TArray<FGuid> ClientIds = LiveSession->GetSession().GetSessionClientEndpointIds();
	ClientIds.Remove(InEventContext.SourceEndpointId);
	LiveSession->GetSession().SendCustomEvent(InEvent, ClientIds, EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
}

void FConcertServerSequencerManager::HandleSequencerTimeAdjustmentEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerTimeAdjustmentEvent& InEvent)
{
	// Verify that we have sequencers with the given SequenceObjectPath open.
	//
	FConcertOpenSequencerState* SequencerState = SequencerStates.Find(*InEvent.SequenceObjectPath);
	if (SequencerState)
	{
		// Forward the message to the other clients
		TArray<FGuid> ClientIds = LiveSession->GetSession().GetSessionClientEndpointIds();
		ClientIds.Remove(InEventContext.SourceEndpointId);
		LiveSession->GetSession().SendCustomEvent(InEvent, ClientIds, EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
	}
}

void FConcertServerSequencerManager::HandleSequencerCloseEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerCloseEvent& InEvent)
{
	FConcertOpenSequencerState* SequencerState = SequencerStates.Find(*InEvent.SequenceObjectPath);
	if (SequencerState)
	{
		SequencerState->ClientEndpointIds.Remove(InEventContext.SourceEndpointId);
		// Forward a normal close event to clients
		const int32 NumOpen = SequencerState->ClientEndpointIds.Num();
		FConcertSequencerCloseEvent CloseEvent;
		CloseEvent.bControllerClose = NumOpen != 0 && InEvent.bControllerClose;
		CloseEvent.EditorsWithSequencerOpened = NumOpen;
		CloseEvent.SequenceObjectPath = InEvent.SequenceObjectPath;
		LiveSession->GetSession().SendCustomEvent(CloseEvent, LiveSession->GetSession().GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
		if (NumOpen == 0)
		{
			SequencerStates.Remove(*InEvent.SequenceObjectPath);
		}
	}
}

void FConcertServerSequencerManager::HandleSequencerPreloadRequestEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerPreloadRequest& InEvent)
{
	const FGuid& RequestClient = InEventContext.SourceEndpointId;
	const bool bClientWantsPreloaded = InEvent.bShouldBePreloaded;

	UE_LOGFMT(LogConcert, Verbose,
		"FConcertServerSequencerManager: Preload request from client {Client} to {AddOrRemove} {NumSequences} sequences",
		*RequestClient.ToString(),
		bClientWantsPreloaded ? TEXT("add") : TEXT("remove"),
		InEvent.SequenceObjectPaths.Num());

	// Represents the net result to broadcast, if any. Contains only sequences
	// which gained their first, or lost their last, referencer.
	FConcertSequencerPreloadRequest OutChanges;
	OutChanges.bShouldBePreloaded = bClientWantsPreloaded;

	for (const FTopLevelAssetPath& SequenceObjectPath : InEvent.SequenceObjectPaths)
	{
		if (bClientWantsPreloaded)
		{
			const bool bAddedFirstReferencer = AddSequencePreloadForClient(RequestClient, SequenceObjectPath);
			if (bAddedFirstReferencer)
			{
				OutChanges.SequenceObjectPaths.Add(SequenceObjectPath);
			}
		}
		else
		{
			const bool bRemovedLastReferencer = RemoveSequencePreloadForClient(RequestClient, SequenceObjectPath);
			if (bRemovedLastReferencer)
			{
				OutChanges.SequenceObjectPaths.Add(SequenceObjectPath);
				ClientPreloadStatuses.Remove(SequenceObjectPath);
			}
		}
	}

	if (OutChanges.SequenceObjectPaths.Num() > 0)
	{
		for (const FTopLevelAssetPath& SequenceObjectPath : OutChanges.SequenceObjectPaths)
		{
			UE_LOGFMT(LogConcert, Verbose,
				"FConcertServerSequencerManager: Sequence {Path} {AddedOrRemoved} preload set",
				SequenceObjectPath.ToString(),
				bClientWantsPreloaded ? TEXT("added to") : TEXT("removed from"));
		}

		LiveSession->GetSession().SendCustomEvent(OutChanges, LiveSession->GetSession().GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
	}
}

void FConcertServerSequencerManager::HandleSequencerPreloadStatusEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerPreloadAssetStatusMap& InEvent)
{
	// Update our cached status, and then forward the change to other clients.
	const FGuid& Sender = InEventContext.SourceEndpointId;
	ClientPreloadStatuses.UpdateFrom(Sender, InEvent);

	FConcertSequencerPreloadClientStatusMap ForwardEvent;
	ForwardEvent.UpdateFrom(Sender, InEvent);
	TArray<FGuid> ForwardRecipients = LiveSession->GetSession().GetSessionClientEndpointIds();
	ForwardRecipients.RemoveSingle(InEventContext.SourceEndpointId);
	LiveSession->GetSession().SendCustomEvent(ForwardEvent, ForwardRecipients, EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
}

void FConcertServerSequencerManager::HandleWorkspaceSyncAndFinalizeCompletedEvent(const FConcertSessionContext& InEventContext, const FConcertWorkspaceSyncAndFinalizeCompletedEvent& InEvent)
{
	FConcertSequencerStateSyncEvent SequencerStateSyncEvent;
	for (const auto& Pair : SequencerStates)
	{
		FConcertSequencerState OutState = Pair.Value.State;
		OutState.TakeData = Pair.Value.TakeData;
		SequencerStateSyncEvent.SequencerStates.Emplace(MoveTemp(OutState));
	}

	LiveSession->GetSession().SendCustomEvent(SequencerStateSyncEvent, InEventContext.SourceEndpointId, EConcertMessageFlags::ReliableOrdered);
}

void FConcertServerSequencerManager::HandleSessionClientChanged(IConcertServerSession& InSession, EConcertClientStatus InClientStatus, const FConcertSessionClientInfo& InClientInfo)
{
	check(&InSession == &LiveSession->GetSession());

	// Remove the client from all open sequences
	if (InClientStatus == EConcertClientStatus::Disconnected)
	{
		for (auto It = SequencerStates.CreateIterator(); It; ++It)
		{
			It->Value.ClientEndpointIds.Remove(InClientInfo.ClientEndpointId);
			const int32 NumOpen = It->Value.ClientEndpointIds.Num();
			// Forward the close event to clients
			FConcertSequencerCloseEvent CloseEvent;
			CloseEvent.EditorsWithSequencerOpened = NumOpen;
			CloseEvent.SequenceObjectPath = It->Key.ToString();
			LiveSession->GetSession().SendCustomEvent(CloseEvent, LiveSession->GetSession().GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered|EConcertMessageFlags::UniqueId);

			if (NumOpen == 0)
			{
				It.RemoveCurrent();
			}
		}
	}

	// Newly connected clients need to be sent the current set of preloaded sequences.
	// Disconnecting clients need their references removed, which may update other clients.
	if (InClientStatus == EConcertClientStatus::Connected ||
		InClientStatus == EConcertClientStatus::Disconnected)
	{
		FConcertSequencerPreloadRequest PreloadRequest;
		TArray<FGuid> EventRecipients;

		if (InClientStatus == EConcertClientStatus::Connected)
		{
			PreloadRequest.bShouldBePreloaded = true;
			EventRecipients.Add(InClientInfo.ClientEndpointId);
		}
		else
		{
			PreloadRequest.bShouldBePreloaded = false;
			EventRecipients = LiveSession->GetSession().GetSessionClientEndpointIds();
		}

		for (TMap<FTopLevelAssetPath, TSet<FGuid>>::TIterator It = PreloadRequesters.CreateIterator(); It; ++It)
		{
			if (InClientStatus == EConcertClientStatus::Connected)
			{
				ensure(It->Value.Num() > 0);
				PreloadRequest.SequenceObjectPaths.Add(It->Key);
			}
			else
			{
				if (It->Value.Remove(InClientInfo.ClientEndpointId))
				{
					if (It->Value.Num() == 0)
					{
						PreloadRequest.SequenceObjectPaths.Add(It->Key);
						It.RemoveCurrent();
					}
				}
			}
		}

		if (PreloadRequest.SequenceObjectPaths.Num() > 0)
		{
			for (const FTopLevelAssetPath& SequenceObjectPath : PreloadRequest.SequenceObjectPaths)
			{
				if (InClientStatus == EConcertClientStatus::Connected)
				{
					UE_LOGFMT(LogConcert, Verbose,
						"FConcertServerSequencerManager: Client connected; notifying preload set contains sequence '{Path}'",
						SequenceObjectPath.ToString());
				}
				else
				{
					UE_LOGFMT(LogConcert, Verbose,
						"FConcertServerSequencerManager: Client disconnected; last reference to '{Path}' was released, removed from preload set",
						SequenceObjectPath.ToString());
				}
			}

			LiveSession->GetSession().SendCustomEvent(PreloadRequest, EventRecipients, EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
		}
	}

	// Newly connected clients won't be sent the Sequencer state sync event
	// until they have synced and finalized their workspace since an open
	// sequence could have been created by a transaction in the activity
	// stream.
}

bool FConcertServerSequencerManager::AddSequencePreloadForClient(const FGuid& RequestClient, const FTopLevelAssetPath& SequenceObjectPath)
{
	bool bAddedFirstReferencer = false;

	TSet<FGuid>& Requesters = PreloadRequesters.FindOrAdd(SequenceObjectPath);
	bool bAlreadyInSet = false;
	Requesters.Add(RequestClient, &bAlreadyInSet);
	if (!bAlreadyInSet)
	{
		if (Requesters.Num() == 1)
		{
			bAddedFirstReferencer = true;
		}
	}
	else
	{
		UE_LOGFMT(LogConcert, Warning, "FConcertServerSequencerManager: Client {Client} requested redundant add preload for sequence {Path}",
			RequestClient.ToString(), SequenceObjectPath.ToString());
	}

	return bAddedFirstReferencer;
}

bool FConcertServerSequencerManager::RemoveSequencePreloadForClient(const FGuid& RequestClient, const FTopLevelAssetPath& SequenceObjectPath)
{
	bool bRemovedLastReferencer = false;

	TSet<FGuid>* MaybeRequesters = PreloadRequesters.Find(SequenceObjectPath);
	if (MaybeRequesters && MaybeRequesters->Remove(RequestClient))
	{
		if (MaybeRequesters->Num() == 0)
		{
			// Removed last reference.
			MaybeRequesters = nullptr;
			PreloadRequesters.Remove(SequenceObjectPath);
			bRemovedLastReferencer = true;
		}
	}
	else
	{
		UE_LOGFMT(LogConcert, Warning, "FConcertServerSequencerManager: Client {Client} attempted invalid release preload for sequence {Path}",
			RequestClient.ToString(), SequenceObjectPath.ToString());
	}

	return bRemovedLastReferencer;
}
