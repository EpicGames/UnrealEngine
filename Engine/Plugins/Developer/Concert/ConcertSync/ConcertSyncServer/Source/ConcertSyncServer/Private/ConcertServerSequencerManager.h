// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessages.h"
#include "ConcertSequencerMessages.h"
#include "ConcertWorkspaceMessages.h"

struct FConcertSessionContext;
class IConcertServerSession;
class FConcertSyncServerLiveSession;

class FConcertServerSequencerManager
{
public:
	explicit FConcertServerSequencerManager(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession);
	~FConcertServerSequencerManager();

	/** Bind this manager to the server session. */
	void BindSession(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession);

	/** Unbind the manager from its currently bound session. */
	void UnbindSession();

private:
	/** Handler for the sequencer state updated event. */
	void HandleSequencerStateEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerStateEvent& InEvent);

	/** Handler for the sequencer open event. */
	void HandleSequencerOpenEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerOpenEvent& InEvent);

	/** Handler for the sequencer close event. */
	void HandleSequencerCloseEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerCloseEvent& InEvent);

	/** Handler for the sequencer time adjustment event. */
	void HandleSequencerTimeAdjustmentEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerTimeAdjustmentEvent& InEvent);

	/** Handler for sequence preloading requests. */
	void HandleSequencerPreloadRequestEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerPreloadRequest& InEvent);

	/** Handler for sequence preloading status updates. */
	void HandleSequencerPreloadStatusEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerPreloadAssetStatusMap& InEvent);

	/** Handler for the workspace sync and finalize completed event. */
	void HandleWorkspaceSyncAndFinalizeCompletedEvent(const FConcertSessionContext& InEventContext, const FConcertWorkspaceSyncAndFinalizeCompletedEvent& InEvent);

	/** Handler for the session clients changed event. */
	void HandleSessionClientChanged(IConcertServerSession& InSession, EConcertClientStatus InClientStatus, const FConcertSessionClientInfo& InClientInfo);


	/** Adds the specified client endpoint as a referencer for the specified sequence, and returns true if that was the first reference for that sequence. */
	bool AddSequencePreloadForClient(const FGuid& RequestClient, const FTopLevelAssetPath& SequenceObjectPath);

	/** Adds the specified client endpoint as a referencer for the specified sequence, and returns true if that was the last remaining reference for that sequence. */
	bool RemoveSequencePreloadForClient(const FGuid& RequestClient, const FTopLevelAssetPath& SequenceObjectPath);


	struct FConcertOpenSequencerState
	{
		/** Client Endpoints that have this Sequence opened. */
		TArray<FGuid> ClientEndpointIds;

		/** Current state of the Sequence. */
		FConcertSequencerState State;

		/**
		 * In the case that the SequenceObjectPath points to a take preset. We capture the preset data
		 * into a payload that can be applied to take that we are going to open. We store it in the state
		 * so that we can play it back when new users join.
		 */
		UPROPERTY()
		FConcertByteArray TakeData;
	};

	/** Map of all currently opened Sequencer in a session, locally opened or not. */
	TMap<FName, FConcertOpenSequencerState> SequencerStates;

	/** Live session tracked by this manager. */
	TSharedPtr<FConcertSyncServerLiveSession> LiveSession;

	/** Map of level sequences indicating which client endpoints have requested preloading for them. */
	TMap<FTopLevelAssetPath, TSet<FGuid>> PreloadRequesters;

	/** Map of clients' individual progress preloading specific sequences. */
	FConcertSequencerPreloadClientStatusMap ClientPreloadStatuses;
};
