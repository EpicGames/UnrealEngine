// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerSyncCommandQueue.h"
#include "ConcertLogGlobal.h"
#include "HAL/PlatformTime.h"

FConcertServerSyncCommandQueue::FConcertServerSyncCommandQueue()
{
	RegisterEndpoint(GlobalGuid);
}

void FConcertServerSyncCommandQueue::RegisterEndpoint(const FGuid& InEndpointId)
{
	QueuedSyncCommands.FindOrAdd(InEndpointId);
}

void FConcertServerSyncCommandQueue::UnregisterEndpoint(const FGuid& InEndpointId)
{
	QueuedSyncCommands.Remove(InEndpointId);
}

void FConcertServerSyncCommandQueue::SetCommandProcessingMethod(const FGuid& InEndpointId, const ESyncCommandProcessingMethod InProcessingMethod)
{
	FEndpointSyncCommandQueue& EndpointSyncQueue = QueuedSyncCommands.FindChecked(InEndpointId);
	EndpointSyncQueue.ProcessingMethod = InProcessingMethod;
}

void FConcertServerSyncCommandQueue::QueueCommand(const FSyncCommand& InCommand)
{
	QueueCommand(TArrayView<const FGuid>(&GlobalGuid, 1), InCommand);
}

void FConcertServerSyncCommandQueue::QueueCommand(const FGuid& InEndpointId, const FSyncCommand& InCommand)
{
	QueueCommand(TArrayView<const FGuid>(&InEndpointId, 1), InCommand);
}

void FConcertServerSyncCommandQueue::QueueCommand(TArrayView<const FGuid> InEndpointIds, const FSyncCommand& InCommand)
{
	for (const FGuid& EndpointId : InEndpointIds)
	{
		FEndpointSyncCommandQueue& EndpointSyncQueue = QueuedSyncCommands.FindChecked(EndpointId);
		EndpointSyncQueue.CommandQueue.Add(InCommand);
	}
}

void FConcertServerSyncCommandQueue::ProcessQueue(const double InTimeLimitSeconds)
{
	SCOPED_CONCERT_TRACE(FConcertServerSyncCommandQueue_ProcessQueue);
	TArray<FGuid> TimeSlicedEndpointIds;

	double ProcessStartTimeSeconds = FPlatformTime::Seconds();

	// First pass process all non-time-sliced queues
	for (auto& QueuedSyncCommandsPair : QueuedSyncCommands)
	{
		const FGuid& EndpointId = QueuedSyncCommandsPair.Key;
		FEndpointSyncCommandQueue& EndpointSyncQueue = QueuedSyncCommandsPair.Value;

		if (EndpointSyncQueue.ProcessingMethod == ESyncCommandProcessingMethod::ProcessTimeSliced)
		{
			// Process time-sliced endpoints later
			if (EndpointSyncQueue.CommandQueue.Num() > 0)
			{
				TimeSlicedEndpointIds.Add(EndpointId);
			}
			continue;
		}

		check(EndpointSyncQueue.ProcessingMethod == ESyncCommandProcessingMethod::ProcessAll);

		// Process this queue to completion
		for (int32 CommandIndex = 0; CommandIndex < EndpointSyncQueue.CommandQueue.Num(); ++CommandIndex)
		{
			EndpointSyncQueue.CommandQueue[CommandIndex](FSyncCommandContext{CommandIndex, EndpointSyncQueue.CommandQueue.Num()}, EndpointId);
		}
		EndpointSyncQueue.CommandQueue.Reset();
	}

	// Second pass processes time-sliced queues until we run out-of-time, but still ensures that at least one command is sent to each endpoint
	while (TimeSlicedEndpointIds.Num() > 0)
	{
		for (auto TimeSlicedEndpointIdIter = TimeSlicedEndpointIds.CreateIterator(); TimeSlicedEndpointIdIter; ++TimeSlicedEndpointIdIter)
		{
			FEndpointSyncCommandQueue& EndpointSyncQueue = QueuedSyncCommands.FindChecked(*TimeSlicedEndpointIdIter);
			check(EndpointSyncQueue.CommandQueue.Num() > 0);

			EndpointSyncQueue.CommandQueue[0](FSyncCommandContext{0, EndpointSyncQueue.CommandQueue.Num()}, *TimeSlicedEndpointIdIter);
			EndpointSyncQueue.CommandQueue.RemoveAt(0, EAllowShrinking::No);
			if (EndpointSyncQueue.CommandQueue.Num() == 0)
			{
				TimeSlicedEndpointIdIter.RemoveCurrent();
			}
		}

		const double ProcessingTimeSeconds = FPlatformTime::Seconds() - ProcessStartTimeSeconds;
		if (ProcessingTimeSeconds > InTimeLimitSeconds)
		{
			// Out-of-time, bail
			break;
		}
	}
}

bool FConcertServerSyncCommandQueue::IsQueueEmpty(const FGuid& InEndpointId) const
{
	const FEndpointSyncCommandQueue& EndpointSyncQueue = QueuedSyncCommands.FindChecked(InEndpointId);
	return EndpointSyncQueue.CommandQueue.Num() == 0;
}

void FConcertServerSyncCommandQueue::ClearQueue(const FGuid& InEndpointId)
{
	FEndpointSyncCommandQueue& EndpointSyncQueue = QueuedSyncCommands.FindChecked(InEndpointId);
	EndpointSyncQueue.CommandQueue.Reset();
}

void FConcertServerSyncCommandQueue::ClearQueue()
{
	for (auto& QueuedSyncCommandsPair : QueuedSyncCommands)
	{
		FEndpointSyncCommandQueue& EndpointSyncQueue = QueuedSyncCommandsPair.Value;
		EndpointSyncQueue.CommandQueue.Reset();
	}
}

void FConcertServerSyncCommandQueue::ResetQueue()
{
	QueuedSyncCommands.Reset();
}
