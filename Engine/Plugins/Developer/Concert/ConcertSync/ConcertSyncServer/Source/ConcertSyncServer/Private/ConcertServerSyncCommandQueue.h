// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Containers/ArrayView.h"

/**
 * Queue of ordered events that need to be synced to another client.
 */
class FConcertServerSyncCommandQueue
{
public:
	/**
	 * Methods by which queued commands can be processed during ProcessQueue.
	 */
	enum ESyncCommandProcessingMethod : uint8
	{
		/** Process all queued commands, regardless of how long it takes */
		ProcessAll,
		/** Process queued commands up-to the given time limit (will always process at least one command per-client) */
		ProcessTimeSliced,
	};

	/**
	 * Context object passed to FSyncCommand.
	 */
	struct FSyncCommandContext
	{
		/** Index of this command in its queue */
		int32 CommandIndex;

		/** Total number of commands in this queue */
		int32 NumCommands;

		/** Get the number remaining commands in the queue */
		int32 GetNumRemainingCommands() const
		{
			return NumCommands - CommandIndex - 1;
		}
	};

	/**
	 * Function representing a sync command.
	 * Is passed the endpoint ID of the target.
	 */
	typedef TFunction<void(const FSyncCommandContext&, const FGuid&)> FSyncCommand;

	/**
	 * Constructor
	 */
	FConcertServerSyncCommandQueue();

	/**
	 * Non-copyable.
	 */
	FConcertServerSyncCommandQueue(const FConcertServerSyncCommandQueue&) = delete;
	FConcertServerSyncCommandQueue& operator=(const FConcertServerSyncCommandQueue&) = delete;

	/**
	 * Register this endpoint with the queue.
	 * @note This must be called before assigning work or properties to the endpoint.
	 */
	void RegisterEndpoint(const FGuid& InEndpointId);

	/**
	 * Unregister this endpoint from the queue.
	 */
	void UnregisterEndpoint(const FGuid& InEndpointId);

	/**
	 * Set the command processing method for the given endpoint.
	 */
	void SetCommandProcessingMethod(const FGuid& InEndpointId, const ESyncCommandProcessingMethod InProcessingMethod);

	/**
	 * Queue a command to be run that is not attached to any particular endpoint.
	 * the endpoint is handled within the FSyncCommand.
	 */
	void QueueCommand(const FSyncCommand& InCommand);

	/**
	 * Queue a command to process for the given endpoint.
	 */
	void QueueCommand(const FGuid& InEndpointId, const FSyncCommand& InCommand);

	/**
	 * Queue a command to process for the given endpoints.
	 */
	void QueueCommand(TArrayView<const FGuid> InEndpointIds, const FSyncCommand& InCommand);

	/**
	 * Process the queue, attempting to limit time-sliced endpoints to the given time limit.
	 * @note Will always process at least one command for each time-sliced endpoint.
	 */
	void ProcessQueue(const double InTimeLimitSeconds = 0.0);

	/**
	 * Check whether the queue for the given endpoint is empty.
	 */
	bool IsQueueEmpty(const FGuid& InEndpointId) const;

	/**
	 * Clear the queue for the given endpoint.
	 */
	void ClearQueue(const FGuid& InEndpointId);

	/**
	 * Clear the queue for all endpoints.
	 */
	void ClearQueue();

	/**
	 * Reset the queue back to its empty state with no endpoints registered.
	 */
	void ResetQueue();

	const FGuid GlobalGuid = FGuid::NewGuid();

private:
	struct FEndpointSyncCommandQueue
	{
		ESyncCommandProcessingMethod ProcessingMethod = ESyncCommandProcessingMethod::ProcessAll;
		TArray<FSyncCommand> CommandQueue;
	};

	TMap<FGuid, FEndpointSyncCommandQueue> QueuedSyncCommands;
};
