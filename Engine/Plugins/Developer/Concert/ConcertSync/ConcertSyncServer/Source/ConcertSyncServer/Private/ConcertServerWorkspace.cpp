// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerWorkspace.h"

#include "ConcertMessageData.h"
#include "IConcertSession.h"
#include "IConcertFileSharingService.h"
#include "ConcertSyncServerLiveSession.h"
#include "ConcertServerSyncCommandQueue.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertTransactionEvents.h"
#include "ConcertServerDataStore.h"
#include "ConcertLogGlobal.h"
#include "ConcertPackageEvents.h"
#include "ConcertUtil.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Serialization/MemoryReader.h"

#include "Algo/Transform.h"

FConcertServerWorkspace::FConcertServerWorkspace(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession, TSharedPtr<IConcertFileSharingService> InFileSharingService)
	: FileSharingService(InFileSharingService ? InFileSharingService->Clone() : nullptr)
	, ReplicationWorkspace(
		InLiveSession->GetSessionDatabase(),
		UE::ConcertSyncServer::FFindSessionClient::CreateLambda([this](const FGuid& EndpointId)
		{
			FConcertSessionClientInfo Info;
			return LiveSession->GetSession().FindSessionClient(EndpointId, Info) ? Info : TOptional<FConcertSessionClientInfo>{};
		}),
		UE::ConcertSyncServer::FShouldIgnoreClientActivityOnRestore::CreateRaw(this, &FConcertServerWorkspace::ShouldIgnoreClientActivityOnRestore)
		)
{
	BindSession(InLiveSession);
	// Sync replication activity back to clients when it is added to the database
	ReplicationWorkspace.OnAddReplicationActivity().AddRaw(this, &FConcertServerWorkspace::OnAddReplicationActivity);
}

FConcertServerWorkspace::~FConcertServerWorkspace()
{
	UnbindSession();
}

void FConcertServerWorkspace::BindSession(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession)
{
	check(InLiveSession->IsValidSession());

	UnbindSession();
	LiveSession = InLiveSession;

	// Create Sync Command Queue
	SyncCommandQueue = MakeShared<FConcertServerSyncCommandQueue>();

	// Create Locked Resources
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableLocking))
	{
		LockedResources = MakeUnique<FLockedResources>();
	}

	// Create Data Store
	DataStore = MakeUnique<FConcertServerDataStore>(LiveSession.ToSharedRef());

	// Register Tick events
	LiveSession->GetSession().OnTick().AddRaw(this, &FConcertServerWorkspace::HandleTick);

	// Register Client Change events
	LiveSession->GetSession().OnSessionClientChanged().AddRaw(this, &FConcertServerWorkspace::HandleSessionClientChanged);
	
	LiveSession->GetSession().OnConcertMessageAcknowledgementReceived().AddRaw(this, &FConcertServerWorkspace::HandleSessionAcknowledgementReceived);

	LiveSession->GetSession().RegisterCustomEventHandler<FConcertWorkspaceSyncRequestedEvent>(this, &FConcertServerWorkspace::HandleSyncRequestedEvent);

	LiveSession->GetSession().RegisterCustomEventHandler<FConcertPackageTransmissionStartEvent>(this, &FConcertServerWorkspace::HandlePackageTransmissionStartEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertPackageUpdateEvent>(this, &FConcertServerWorkspace::HandlePackageUpdateEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertPlaySessionEvent>(this, &FConcertServerWorkspace::HandlePlaySessionEvent);

	LiveSession->GetSession().RegisterCustomEventHandler<FConcertTransactionFinalizedEvent>(this, &FConcertServerWorkspace::HandleTransactionFinalizedEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertTransactionSnapshotEvent>(this, &FConcertServerWorkspace::HandleTransactionSnapshotEvent);

	LiveSession->GetSession().RegisterCustomRequestHandler<FConcertResourceLockRequest, FConcertResourceLockResponse>(this, &FConcertServerWorkspace::HandleResourceLockRequest);
	LiveSession->GetSession().RegisterCustomRequestHandler<FConcertSyncEventRequest, FConcertSyncEventResponse>(this, &FConcertServerWorkspace::HandleSyncEventRequest);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertIgnoreActivityStateChangedEvent>(this, &FConcertServerWorkspace::HandleIgnoredActivityStateChanged);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertServerLogging>(this, &FConcertServerWorkspace::HandleServerLoggingEvent);

	if (FileSharingService)
	{
		FString BasePath = FileSharingService->GetPath();
		FString FullPath = BasePath / LiveSession->GetSession().GetName() / TEXT("Server") / LiveSession->GetSession().GetId().ToString();
		FileSharingService->SetPath(FullPath);
	}
}

void FConcertServerWorkspace::UnbindSession()
{
	if (LiveSession.IsValid())
	{
		if (FileSharingService)
		{
			FileSharingService->Delete(FileSharingService->GetPath());
		}
		
		// Destroy Sync Command Queue
		SyncCommandQueue.Reset();

		// Destroy Locked Resources
		LockedResources.Reset();

		// Destroy Data Store
		DataStore.Reset();

		// Unregister Tick events
		LiveSession->GetSession().OnTick().RemoveAll(this);

		// Unregister Client Change events
		LiveSession->GetSession().OnSessionClientChanged().RemoveAll(this);

		LiveSession->GetSession().OnConcertMessageAcknowledgementReceived().RemoveAll(this);

		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertWorkspaceSyncRequestedEvent>(this);

		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertPackageTransmissionStartEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertPackageUpdateEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertPlaySessionEvent>(this);

		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertTransactionFinalizedEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertTransactionSnapshotEvent>(this);

		LiveSession->GetSession().UnregisterCustomRequestHandler<FConcertResourceLockRequest>();
		LiveSession.Reset();
	}
}

void FConcertServerWorkspace::HandleTick(IConcertServerSession& InSession, float InDeltaTime)
{
	SCOPED_CONCERT_TRACE(FConcertServerWorkspace_HandleTick);
	check(&LiveSession->GetSession() == &InSession);

	static const double SyncFrameLimitSeconds = 1.0 / 60;
	SyncCommandQueue->ProcessQueue(SyncFrameLimitSeconds);

	for (auto ManualSyncEndpointIter = ManualSyncEndpoints.CreateIterator(); ManualSyncEndpointIter; ++ManualSyncEndpointIter)
	{
		if (SyncCommandQueue->IsQueueEmpty(*ManualSyncEndpointIter))
		{
			LiveSession->GetSession().SendCustomEvent(FConcertWorkspaceSyncCompletedEvent(), *ManualSyncEndpointIter, EConcertMessageFlags::ReliableOrdered);
			SyncCommandQueue->SetCommandProcessingMethod(*ManualSyncEndpointIter, FConcertServerSyncCommandQueue::ESyncCommandProcessingMethod::ProcessAll);
			ManualSyncEndpointIter.RemoveCurrent();
		}
	}

	LiveSession->GetSessionDatabase().UpdateAsynchronousTasks();
}


void FConcertServerWorkspace::HandleServerLoggingEvent(const FConcertSessionContext& Context, const FConcertServerLogging& InEvent)
{
	ConcertUtil::SetVerboseLogging(InEvent.bLoggingEnabled);
}

void FConcertServerWorkspace::HandleSessionClientChanged(IConcertServerSession& InSession, EConcertClientStatus InClientStatus, const FConcertSessionClientInfo& InClientInfo)
{
	check(&LiveSession->GetSession() == &InSession);

	if (InClientStatus == EConcertClientStatus::Connected || InClientStatus == EConcertClientStatus::Updated)
	{
		FConcertSyncEndpointData SyncEndpointData;
		SyncEndpointData.ClientInfo = InClientInfo.ClientInfo;
		SetEndpoint(InClientInfo.ClientEndpointId, SyncEndpointData);
	}

	if (InClientStatus == EConcertClientStatus::Connected)
	{
		SyncCommandQueue->RegisterEndpoint(InClientInfo.ClientEndpointId);

		// Add the connection activity
		if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableConnectionHistory))
		{
			FConcertSyncConnectionActivity ConnectionActivity;
			ConnectionActivity.EndpointId = InClientInfo.ClientEndpointId;
			ConnectionActivity.EventData.ConnectionEventType = EConcertSyncConnectionEventType::Connected;
			ConnectionActivity.EventSummary.SetTypedPayload(FConcertSyncConnectionActivitySummary::CreateSummaryForEvent(ConnectionActivity.EventData));
			ConnectionActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InClientInfo.ClientEndpointId);
			AddConnectionActivity(ConnectionActivity);
		}
	}
	else if (InClientStatus == EConcertClientStatus::Disconnected)
	{
		UnlockAllWorkspaceResources(InClientInfo.ClientEndpointId);
		HandleEndPlaySessions(InClientInfo.ClientEndpointId);

		// Add the disconnection activity
		if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableConnectionHistory))
		{
			FConcertSyncConnectionActivity ConnectionActivity;
			ConnectionActivity.EndpointId = InClientInfo.ClientEndpointId;
			ConnectionActivity.EventData.ConnectionEventType = EConcertSyncConnectionEventType::Disconnected;
			ConnectionActivity.EventSummary.SetTypedPayload(FConcertSyncConnectionActivitySummary::CreateSummaryForEvent(ConnectionActivity.EventData));
			ConnectionActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InClientInfo.ClientEndpointId);
			AddConnectionActivity(ConnectionActivity);
		}

		LiveSyncEndpoints.Remove(InClientInfo.ClientEndpointId);
		ManualSyncEndpoints.Remove(InClientInfo.ClientEndpointId);
		SyncCommandQueue->UnregisterEndpoint(InClientInfo.ClientEndpointId);

		IgnoredActivityClients.Remove(InClientInfo.ClientEndpointId);
	}
}

namespace UE::ConcertSyncServer::Private
{
	template<typename TSyncActivityType>
	static void ExtractSyncActivity(const TSharedRef<IConcertMessage>& AckedMessage, TFunctionRef<void(const TSyncActivityType& Activity)> Callback)
	{
		if (AckedMessage->GetMessageType() == FConcertSession_CustomEvent::StaticStruct())
		{
			const FConcertSession_CustomEvent* CustomEvent = reinterpret_cast<const FConcertSession_CustomEvent*>(AckedMessage->ConstructMessage());
			if (CustomEvent->SerializedPayload.IsTypeChildOf<FConcertWorkspaceSyncActivityEvent>())
			{
				FConcertWorkspaceSyncActivityEvent SyncActivity;
				CustomEvent->SerializedPayload.GetTypedPayload(SyncActivity);
				if (SyncActivity.Activity.IsTypeChildOf(TBaseStructure<TSyncActivityType>::Get()))
				{
					TSyncActivityType Result;
					SyncActivity.Activity.GetTypedPayload(Result);
					Callback(Result);
				}
			}
		}
	}
}

void FConcertServerWorkspace::HandleSessionAcknowledgementReceived(const FConcertEndpointContext& LocalEndpoint, const FConcertEndpointContext& RemoteEndpoint, const TSharedRef<IConcertMessage>& AckedMessage, const FConcertMessageContext& MessageContext) const
{
	UE::ConcertSyncServer::Private::ExtractSyncActivity<FConcertSyncPackageActivity>(AckedMessage, [&RemoteEndpoint, &AckedMessage](const FConcertSyncPackageActivity& PackageActivity)
	{
		if (PackageActivity.EventData.Package.HasPackageData())
		{
			const FGuid TransmissionId = HashEndpointIdAndActivityId(RemoteEndpoint.EndpointId, PackageActivity.ActivityId);
			UE::ConcertSyncCore::ConcertPackageEvents::OnLocalFinishSendPackage().Broadcast({ { TransmissionId, PackageActivity.EventData.Package.Info, RemoteEndpoint.EndpointId }, AckedMessage->GetMessageId() });
		}
	});
}

void FConcertServerWorkspace::HandleIgnoredActivityStateChanged(const FConcertSessionContext& Context, const FConcertIgnoreActivityStateChangedEvent& Event)
{
	if (Event.bIgnore)
	{
		IgnoredActivityClients.Add(Event.EndpointId);
	}
	else
	{
		IgnoredActivityClients.Remove(Event.EndpointId);
	}
}

void FConcertServerWorkspace::HandleSyncRequestedEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncRequestedEvent& Event)
{
	if (Event.bEnableLiveSync)
	{
		LiveSyncEndpoints.AddUnique(Context.SourceEndpointId);
	}
	else
	{
		LiveSyncEndpoints.Remove(Context.SourceEndpointId);
	}

	// Resolve the activity range to sync
	int64 FirstActivityIdToSync = 0;
	int64 NumActivitiesToSync = 0;
	{
		int64 ActivityMaxId = 0;
		LiveSession->GetSessionDatabase().GetActivityMaxId(ActivityMaxId);
		if (!Event.bEnableLiveSync)
		{
			ActivityMaxId = FMath::Min<int64>(ActivityMaxId, Event.LastActivityIdToSync);
		}

		FirstActivityIdToSync = FMath::Max<int64>(1, Event.FirstActivityIdToSync);
		NumActivitiesToSync = FMath::Max<int64>(0, (ActivityMaxId - FirstActivityIdToSync) + 1);
	}

	// Manual sync requests will be time-sliced until they've finished their requested sync
	ManualSyncEndpoints.AddUnique(Context.SourceEndpointId);
	SyncCommandQueue->SetCommandProcessingMethod(Context.SourceEndpointId, FConcertServerSyncCommandQueue::ESyncCommandProcessingMethod::ProcessTimeSliced);

	// Sync all endpoints
	LiveSession->GetSessionDatabase().EnumerateEndpointIds([this, &Context](FGuid InEndpointId)
	{
		SyncCommandQueue->QueueCommand(Context.SourceEndpointId, [this, SyncEndpointId = InEndpointId](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			SendSyncEndpointEvent(InEndpointId, SyncEndpointId, InSyncCommandContext.GetNumRemainingCommands());
		});
		return true;
	});

	// Sync all activity
	LiveSession->GetSessionDatabase().EnumerateActivityIdsWithEventTypesAndFlagsInRange(FirstActivityIdToSync, NumActivitiesToSync, [this, &Context](const int64 InActivityId, const EConcertSyncActivityEventType InEventType, const EConcertSyncActivityFlags InFlags)
	{
		if ((InFlags & EConcertSyncActivityFlags::Muted) != EConcertSyncActivityFlags::None)
		{
			return true;
		}
		
		SyncCommandQueue->QueueCommand(Context.SourceEndpointId, [this, SyncActivityId = InActivityId, SyncEventType = InEventType](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			static_assert(static_cast<uint8>(EConcertSyncActivityEventType::Count) == 6, "If you added an EConcertSyncActivityEventType entry, update this switch");
			switch (SyncEventType)
			{
			case EConcertSyncActivityEventType::Connection:
				SendSyncConnectionActivityEvent(InEndpointId, SyncActivityId, InSyncCommandContext.GetNumRemainingCommands());
				break;

			case EConcertSyncActivityEventType::Lock:
				SendSyncLockActivityEvent(InEndpointId, SyncActivityId, InSyncCommandContext.GetNumRemainingCommands());
				break;

			case EConcertSyncActivityEventType::Transaction:
				SendSyncTransactionActivityEvent(InEndpointId, SyncActivityId, InSyncCommandContext.GetNumRemainingCommands());
				break;

			case EConcertSyncActivityEventType::Package:
			{
				TOptional<FConcertWorkspaceSyncActivityEvent> SyncEvent = MakeSyncActivityEvent(SyncActivityId);
				if(SyncEvent)
				{
					SyncEvent->NumRemainingSyncEvents = InSyncCommandContext.GetNumRemainingCommands();
					SendSyncPackageActivityEvent(SyncEvent.GetValue(), InEndpointId);
				}
				break;
			}
			case EConcertSyncActivityEventType::Replication:
				SendSyncReplicationActivityEvent(InEndpointId, SyncActivityId, InSyncCommandContext.GetNumRemainingCommands());
				break;
			default:
				checkf(false, TEXT("Unhandled EConcertSyncActivityEventType when syncing session activity"));
				break;
			}
		});

		return true;
	});

	// Sync live resource locks
	if (LockedResources)
	{
		SyncCommandQueue->QueueCommand(Context.SourceEndpointId, [this](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			FConcertWorkspaceSyncLockEvent SyncEvent;
			SyncEvent.NumRemainingSyncEvents = InSyncCommandContext.GetNumRemainingCommands();
			Algo::Transform(*LockedResources, SyncEvent.LockedResources, [](const TPair<FName, FLockOwner>& Pair)
			{
				return TPair<FName, FGuid>{ Pair.Key, Pair.Value.EndpointId };
			});
			LiveSession->GetSession().SendCustomEvent(SyncEvent, InEndpointId, EConcertMessageFlags::ReliableOrdered);
		});
	}

	// Sync PIE/SIE play states
	SyncCommandQueue->QueueCommand(Context.SourceEndpointId, [this](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
	{
		for (const TPair<FName, TArray<FPlaySessionInfo>>& PlayInfoPair : ActivePlaySessions)
		{
			for (const FPlaySessionInfo& PlayInfo : PlayInfoPair.Value)
			{
				LiveSession->GetSession().SendCustomEvent(FConcertPlaySessionEvent{ EConcertPlaySessionEventType::BeginPlay, PlayInfo.EndpointId, PlayInfoPair.Key, PlayInfo.bIsSimulating }, InEndpointId, EConcertMessageFlags::ReliableOrdered);
			}
		}
	});
}

void FConcertServerWorkspace::HandlePackageTransmissionStartEvent(const FConcertSessionContext& Context, const FConcertPackageTransmissionStartEvent& Event)
{
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnablePackages)
		&& ensureMsgf(Event.PackageNumBytes > 0, TEXT("Client sent invalid message")))
	{
		// Clients send this before the FConcertPackageUpdateEvent in reliable order so we can be sure we'll trigger OnRemoteBeginSendPackage before OnRemoteFinishSendPackage
		UE::ConcertSyncCore::ConcertPackageEvents::OnRemoteBeginSendPackage().Broadcast({ { Event.TransmissionId, Event.PackageInfo, Context.SourceEndpointId }, Event.PackageNumBytes });
	}
}

void FConcertServerWorkspace::HandlePackageUpdateEvent(const FConcertSessionContext& Context, const FConcertPackageUpdateEvent& Event)
{
	if (!EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnablePackages))
	{
		return;
	}

	// Consider acquiring lock on asset saving an explicit lock
	const bool bLockOwned = LockWorkspaceResource(Event.Package.Info.PackageName, Context.SourceEndpointId, EConcertLockFlags::Temporary);
	if (bLockOwned)
	{
		// If the client has the lock, then add the package activity
		{
			bool bActivityAdded = false;

			// Fill up required fields for the 'base' part of the activity.
			FConcertSyncPackageActivity PackageActivityBasePart;
			PackageActivityBasePart.EndpointId = Context.SourceEndpointId;
			PackageActivityBasePart.EventSummary.SetTypedPayload(FConcertSyncPackageActivitySummary::CreateSummaryForEvent(Event.Package.Info));
			PackageActivityBasePart.bIgnored = ShouldIgnoreClientActivityOnRestore(Context.SourceEndpointId);

			// Fill up required fields for the 'event' part of the activity.
			FConcertPackage Package;
			Package.Info = Event.Package.Info;

			if (Event.Package.Info.PackageUpdateType == EConcertPackageUpdateType::Dummy)
			{
				// Without Live Sync, the clients don't track the ongoing transactions in their 'local' database and can't provide the current transaction event ID when unsaved modifications to a package are discarded (to discard corresponding live transactions on this package).
				if (!EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableLiveSync))
				{
					LiveSession->GetSessionDatabase().GetTransactionMaxEventId(Package.Info.TransactionEventIdAtSave); // Get the last transaction event from the server db.
				}

				if (!Event.Package.HasPackageData()) // The dummy event didn't come with any package data.
				{
					// Attempt to migrate the package data from the current head package revision so that newly synced clients will receive the correct package data.
					LiveSession->GetSessionDatabase().GetPackageDataForRevision(Package.Info.PackageName, [this, &bActivityAdded, &Package, &PackageActivityBasePart](const FConcertPackageInfo& InHeadPackageInfo, FConcertPackageDataStream& InHeadPackageDataStream)
					{
						// Add the new package activity with the head revision data.
						Package.Info.PackageFileExtension = InHeadPackageInfo.PackageFileExtension;
						AddPackageActivity(PackageActivityBasePart, Package.Info, InHeadPackageDataStream);
						bActivityAdded = true;
					});
				}
			}

			if (!bActivityAdded)
			{
				TSharedPtr<FArchive> PackageDataAr = GetPackageDataStream(Event.Package);
				FConcertPackageDataStream PackageDataStream { 
					PackageDataAr.Get(),
					PackageDataAr ? PackageDataAr->TotalSize() : 0,
					Event.Package.PackageData.Bytes.Num() ? &Event.Package.PackageData.Bytes : nullptr
				};
 
				AddPackageActivity(PackageActivityBasePart, Package.Info, PackageDataStream);
			}
		}

		if (Event.Package.HasPackageData())
		{
			UE::ConcertSyncCore::ConcertPackageEvents::OnRemoteFinishSendPackage().Broadcast({ { Event.TransmissionId, Event.Package.Info, Context.SourceEndpointId }, Context.MessageId });
		}
		
		// Explicitly unlock the resource after saving it
		UnlockWorkspaceResource(Event.Package.Info.PackageName, Context.SourceEndpointId, EConcertLockFlags::Explicit);
	}
	else
	{
		if (Event.Package.HasPackageData())
		{
			const UE::ConcertSyncCore::ConcertPackageEvents::FConcertRejectSendPackageParams Params {{{ Event.TransmissionId, Event.Package.Info, Context.SourceEndpointId }, Context.MessageId}};
			UE::ConcertSyncCore::ConcertPackageEvents::OnRejectRemoteSendPackage().Broadcast(Params);
		}
		
		// If the client didn't have the lock, then queue a rejection event so that the client will re-load the head-revision of the package
		SyncCommandQueue->QueueCommand(Context.SourceEndpointId, [this, PackageName = Event.Package.Info.PackageName](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			FConcertPackageRejectedEvent PackageRejectedEvent;
			PackageRejectedEvent.PackageName = PackageName;
			LiveSession->GetSession().SendCustomEvent(PackageRejectedEvent, InEndpointId, EConcertMessageFlags::ReliableOrdered);
		});
	}
}

void FConcertServerWorkspace::HandleTransactionFinalizedEvent(const FConcertSessionContext& InEventContext, const FConcertTransactionFinalizedEvent& InEvent)
{
	if (!EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableTransactions))
	{
		return;
	}

	// Implicitly acquire locks for all objects in the transaction
	TArray<FName> ResourceNames;
	Algo::Transform(InEvent.ExportedObjects, ResourceNames, [](const FConcertExportedObject& InObject)
	{
		// TODO: This isn't always the correct way to build the object path (re: SUBOBJECT_DELIMITER)
		return FName(*FString::Printf(TEXT("%s.%s"), *InObject.ObjectId.ObjectOuterPathName.ToString(), *InObject.ObjectId.ObjectName.ToString()));
	});

	const bool bLockOwned = LockWorkspaceResources(ResourceNames, InEventContext.SourceEndpointId, EConcertLockFlags::Temporary);
	if (bLockOwned)
	{
		// If the client has the lock, then add the transaction activity
		{
			FConcertSyncTransactionActivity TransactionActivity;
			TransactionActivity.EndpointId = InEventContext.SourceEndpointId;
			TransactionActivity.EventData.Transaction = InEvent;
			TransactionActivity.EventSummary.SetTypedPayload(FConcertSyncTransactionActivitySummary::CreateSummaryForEvent(TransactionActivity.EventData));
			TransactionActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InEventContext.SourceEndpointId);
			AddTransactionActivity(TransactionActivity);
		}

		// Implicitly unlock resources in the transaction (TODO: should use sync queue?)
		bool bUnlocked = UnlockWorkspaceResources(ResourceNames, InEventContext.SourceEndpointId, EConcertLockFlags::None);
		check(bUnlocked);
	}
	else
	{
		// If the client didn't have the lock, then queue a rejection event
		FConcertTransactionRejectedEvent TransactionRejectedEvent;
		TransactionRejectedEvent.TransactionId = InEvent.TransactionId;
		LiveSession->GetSession().SendCustomEvent(TransactionRejectedEvent, InEventContext.SourceEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
}

void FConcertServerWorkspace::HandleTransactionSnapshotEvent(const FConcertSessionContext& InEventContext, const FConcertTransactionSnapshotEvent& InEvent)
{
	if (!EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableTransactions))
	{
		return;
	}

	// Implicitly acquire locks for objects in the transaction
	TArray<FName> ResourceNames;
	Algo::Transform(InEvent.ExportedObjects, ResourceNames, [](const FConcertExportedObject& InObject)
	{
		// TODO: This isn't always the correct way to build the object path (re: SUBOBJECT_DELIMITER)
		return FName(*FString::Printf(TEXT("%s.%s"), *InObject.ObjectId.ObjectOuterPathName.ToString(), *InObject.ObjectId.ObjectName.ToString()));
	});

	const bool bLockOwned = LockWorkspaceResources(ResourceNames, InEventContext.SourceEndpointId, EConcertLockFlags::None);
	if (bLockOwned)
	{
		// If the client has the lock, then forward the snapshot
		TArray<FGuid> QueueClientEndpointIds = LiveSession->GetSession().GetSessionClientEndpointIds();
		QueueClientEndpointIds.Remove(InEventContext.SourceEndpointId);
		LiveSession->GetSession().SendCustomEvent(InEvent, QueueClientEndpointIds, EConcertMessageFlags::UniqueId);
	}
	// otherwise do nothing, we will reject the finalized transaction (TODO: send notification back?)
}

void FConcertServerWorkspace::HandlePlaySessionEvent(const FConcertSessionContext& Context, const FConcertPlaySessionEvent& Event)
{
	// Forward this notification onto all clients except the one that entered the play session
	{
		TArray<FGuid> NotifyEndpointIds = LiveSession->GetSession().GetSessionClientEndpointIds();
		NotifyEndpointIds.Remove(Context.SourceEndpointId);
		LiveSession->GetSession().SendCustomEvent(Event, NotifyEndpointIds, EConcertMessageFlags::ReliableOrdered);
	}

	if (Event.EventType == EConcertPlaySessionEventType::BeginPlay)
	{
		HandleBeginPlaySession(Event.PlayPackageName, Event.PlayEndpointId, Event.bIsSimulating);
	}
	else if (Event.EventType == EConcertPlaySessionEventType::EndPlay)
	{
		HandleEndPlaySession(Event.PlayPackageName, Event.PlayEndpointId);
	}
	else
	{
		check(Event.EventType == EConcertPlaySessionEventType::SwitchPlay);
		HandleSwitchPlaySession(Event.PlayPackageName, Event.PlayEndpointId);
	}
}

EConcertSessionResponseCode FConcertServerWorkspace::HandleResourceLockRequest(const FConcertSessionContext& Context, const FConcertResourceLockRequest& Request, FConcertResourceLockResponse& Response)
{
	check(Context.SourceEndpointId == Request.ClientId);
	Response.LockType = Request.LockType;

	switch (Response.LockType)
	{
	case EConcertResourceLockType::Lock:
		LockWorkspaceResources(Request.ResourceNames, Request.ClientId, EConcertLockFlags::Explicit, &Response.FailedResources);
		break;
	case EConcertResourceLockType::Unlock:
		UnlockWorkspaceResources(Request.ResourceNames, Request.ClientId, EConcertLockFlags::Explicit, &Response.FailedResources);
		break;
	default:
		return EConcertSessionResponseCode::InvalidRequest;
	}
	return EConcertSessionResponseCode::Success;
}

EConcertSessionResponseCode FConcertServerWorkspace::HandleSyncEventRequest(const FConcertSessionContext& Context, const FConcertSyncEventRequest& Request, FConcertSyncEventResponse& Response)
{
	if (Request.EventType == EConcertSyncActivityEventType::Transaction)
	{
		FConcertSyncTransactionEvent TransactionEvent;
		if (LiveSession->GetSessionDatabase().GetTransactionEvent(Request.EventId, TransactionEvent, /*bMetaDataOnly*/false)) // The request is meant to retrive the transaction data from a partially sync event.
		{
			Response.Event.SetTypedPayload(TransactionEvent);
			return EConcertSessionResponseCode::Success;
		}
	}
	// else Other event types (package/connection/lock) don't have interesting non-meta-data data to retrieve.

	return EConcertSessionResponseCode::Failed;
}

void FConcertServerWorkspace::HandleBeginPlaySession(const FName InPlayPackageName, const FGuid& InEndpointId, bool bIsSimulating)
{
	TArray<FPlaySessionInfo>& PlaySessionEndpoints = ActivePlaySessions.FindOrAdd(InPlayPackageName);
	PlaySessionEndpoints.AddUnique({InEndpointId, bIsSimulating});
}

void FConcertServerWorkspace::HandleSwitchPlaySession(const FName InPlayPackageName, const FGuid& InEndpointId)
{
	// The client has toggled between PIE/SIE play type.
	if (TArray<FPlaySessionInfo>* PlaySessionInfo = ActivePlaySessions.Find(InPlayPackageName))
	{
		if (FPlaySessionInfo* PlayInfo = PlaySessionInfo->FindByPredicate([InEndpointId](const FPlaySessionInfo& Info) { return InEndpointId == Info.EndpointId; }))
		{
			PlayInfo->bIsSimulating = !PlayInfo->bIsSimulating; // Toggle the status.
		}
	}
}

void FConcertServerWorkspace::HandleEndPlaySession(const FName InPlayPackageName, const FGuid& InEndpointId)
{
	bool bDiscardPackage = false;
	if (TArray<FPlaySessionInfo>* PlaySessionInfo = ActivePlaySessions.Find(InPlayPackageName))
	{
		PlaySessionInfo->RemoveAll([InEndpointId](const FPlaySessionInfo& Info) {return Info.EndpointId == InEndpointId; });
		if (PlaySessionInfo->Num() == 0)
		{
			bDiscardPackage = true;
			ActivePlaySessions.Remove(InPlayPackageName);
		}
	}

	if (bDiscardPackage)
	{
		// Save a dummy package to discard the live transactions for the previous play world
		// Play worlds are never saved, so we don't have to worry about migrating over the previous data here
		FConcertSyncActivity DummyPackageActivity;
		FConcertPackageInfo DummyPackageInfo;
		FConcertPackageDataStream DummyPackageStream;
		DummyPackageActivity.EndpointId = InEndpointId;
		DummyPackageInfo.PackageUpdateType = EConcertPackageUpdateType::Dummy;
		DummyPackageInfo.PackageName = InPlayPackageName;
		LiveSession->GetSessionDatabase().GetTransactionMaxEventId(DummyPackageInfo.TransactionEventIdAtSave);
		DummyPackageActivity.EventSummary.SetTypedPayload(FConcertSyncPackageActivitySummary::CreateSummaryForEvent(DummyPackageInfo));
		DummyPackageActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InEndpointId);
		AddPackageActivity(DummyPackageActivity, DummyPackageInfo, DummyPackageStream);
	}
}

void FConcertServerWorkspace::HandleEndPlaySessions(const FGuid& InEndpointId)
{
	FName PlayPackageName = FindPlaySession(InEndpointId);
	if (!PlayPackageName.IsNone())
	{
		HandleEndPlaySession(PlayPackageName, InEndpointId);

#if DO_CHECK
		// Verify that there are no sessions left using this endpoint (they should only ever end up in a single session)
		PlayPackageName = FindPlaySession(InEndpointId);
		ensureAlwaysMsgf(PlayPackageName.IsNone(), TEXT("Endpoint '%s' has in multiple play sessions!"), *InEndpointId.ToString());
#endif
	}
}

FName FConcertServerWorkspace::FindPlaySession(const FGuid& InEndpointId)
{
	for (const auto& ActivePlaySessionPair : ActivePlaySessions)
	{
		if (ActivePlaySessionPair.Value.ContainsByPredicate([InEndpointId](const FPlaySessionInfo& Info){ return Info.EndpointId == InEndpointId; }))
		{
			return ActivePlaySessionPair.Key;
		}
	}
	return FName();
}

bool FConcertServerWorkspace::LockWorkspaceResource(const FName InResourceName, const FGuid& InLockEndpointId, EConcertLockFlags InLockFlags)
{
	if (!LockedResources)
	{
		return true;
	}

	FLockOwner& Owner = LockedResources->FindOrAdd(InResourceName);
	if (!Owner.EndpointId.IsValid() || EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Force))
	{
		Owner.EndpointId = InLockEndpointId;
		Owner.bExplicit = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Explicit);
		Owner.bTemporary = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Temporary);

		// Add lock activity and notify for non-temporary locks
		if (!Owner.bTemporary)
		{
			FConcertSyncLockActivity LockActivity;
			LockActivity.EndpointId = InLockEndpointId;
			LockActivity.EventData.LockEventType = EConcertSyncLockEventType::Locked;
			LockActivity.EventData.ResourceNames.Add(InResourceName);
			LockActivity.EventSummary.SetTypedPayload(FConcertSyncLockActivitySummary::CreateSummaryForEvent(LockActivity.EventData));
			LockActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InLockEndpointId);
			AddLockActivity(LockActivity);

			FConcertResourceLockEvent LockEvent{ InLockEndpointId, {InResourceName}, EConcertResourceLockType::Lock };
			LiveSession->GetSession().SendCustomEvent(LockEvent, LiveSession->GetSession().GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
		}
	}
	return Owner.EndpointId == InLockEndpointId;
}

bool FConcertServerWorkspace::LockWorkspaceResources(const TArray<FName>& InResourceNames, const FGuid& InLockEndpointId, EConcertLockFlags InLockFlags, TMap<FName, FGuid>* OutFailedRessources)
{
	if (!LockedResources)
	{
		return true;
	}

	const bool bForce = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Force);
	const bool bExplicit = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Explicit);
	const bool bTemporary = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Temporary);

	int32 AcquiredLockCount = 0;
	FConcertResourceLockEvent LockEvent{ InLockEndpointId, {}, EConcertResourceLockType::Lock };
	for (const FName& ResourceName : InResourceNames)
	{
		FLockOwner* Owner = LockedResources->Find(ResourceName);
		if (Owner == nullptr || bForce)
		{
			LockEvent.ResourceNames.Add(ResourceName);
			++AcquiredLockCount;
		}
		else if (Owner->EndpointId == InLockEndpointId)
		{
			++AcquiredLockCount;
		}
		else if (OutFailedRessources != nullptr)
		{
			OutFailedRessources->Add(ResourceName, Owner->EndpointId);
		}
	}

	// if the operation was successful and any new locks were acquired, add them and send an update
	const bool bSuccess = AcquiredLockCount == InResourceNames.Num();
	if (bSuccess && LockEvent.ResourceNames.Num() > 0)
	{
		for (const FName& ResourceName : LockEvent.ResourceNames)
		{
			LockedResources->Add(ResourceName, { InLockEndpointId, bExplicit, bTemporary });
		}

		// Add lock activity and notify for non-temporary locks
		if (!bTemporary)
		{
			FConcertSyncLockActivity LockActivity;
			LockActivity.EndpointId = InLockEndpointId;
			LockActivity.EventData.LockEventType = EConcertSyncLockEventType::Locked;
			LockActivity.EventData.ResourceNames = LockEvent.ResourceNames;
			LockActivity.EventSummary.SetTypedPayload(FConcertSyncLockActivitySummary::CreateSummaryForEvent(LockActivity.EventData));
			LockActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InLockEndpointId);
			AddLockActivity(LockActivity);

			LiveSession->GetSession().SendCustomEvent(LockEvent, LiveSession->GetSession().GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
		}
	}

	return bSuccess;
}

bool FConcertServerWorkspace::UnlockWorkspaceResource(const FName InResourceName, const FGuid& InLockEndpointId, EConcertLockFlags InLockFlags)
{
	if (!LockedResources)
	{
		return true;
	}

	const bool bForce = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Force);
	const bool bExplicit = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Explicit);

	if (const FLockOwner* Owner = LockedResources->Find(InResourceName))
	{
		if (Owner->EndpointId == InLockEndpointId || bForce)
		{
			if (!Owner->bExplicit || bExplicit || bForce)
			{
				const bool bWasTemporary = Owner->bTemporary;
				LockedResources->Remove(InResourceName);
				Owner = nullptr;

				// Add lock activity and notify for non-temporary locks
				if (!bWasTemporary)
				{
					FConcertSyncLockActivity LockActivity;
					LockActivity.EndpointId = InLockEndpointId;
					LockActivity.EventData.LockEventType = EConcertSyncLockEventType::Unlocked;
					LockActivity.EventData.ResourceNames.Add(InResourceName);
					LockActivity.EventSummary.SetTypedPayload(FConcertSyncLockActivitySummary::CreateSummaryForEvent(LockActivity.EventData));
					LockActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InLockEndpointId);
					AddLockActivity(LockActivity);

					FConcertResourceLockEvent LockEvent{ InLockEndpointId, {InResourceName}, EConcertResourceLockType::Unlock };
					LiveSession->GetSession().SendCustomEvent(LockEvent, LiveSession->GetSession().GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
				}
			}
			return true;
		}
	}

	return false;
}

bool FConcertServerWorkspace::UnlockWorkspaceResources(const TArray<FName>& InResourceNames, const FGuid& InLockEndpointId, EConcertLockFlags InLockFlags, TMap<FName, FGuid>* OutFailedRessources)
{
	if (!LockedResources)
	{
		return true;
	}

	const bool bForce = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Force);
	const bool bExplicit = EnumHasAnyFlags(InLockFlags, EConcertLockFlags::Explicit);

	int32 ReleasedLockCount = 0;
	FConcertResourceLockEvent LockEvent{ InLockEndpointId, {}, EConcertResourceLockType::Unlock };
	TArray<FName> ToRemove;
	for (const FName& ResourceName : InResourceNames)
	{
		const FLockOwner Owner = LockedResources->FindRef(ResourceName);
		if (Owner.EndpointId == InLockEndpointId || bForce)
		{
			if (Owner.bExplicit == bExplicit || bForce)
			{
				if (!Owner.bTemporary)
				{
					LockEvent.ResourceNames.Add(ResourceName);
				}
				ToRemove.AddUnique(ResourceName);
			}
			++ReleasedLockCount;
		}
		else if (OutFailedRessources != nullptr)
		{
			OutFailedRessources->Add(ResourceName, Owner.EndpointId);
		}
	}

	for (const FName& ResourceName : ToRemove)
	{
		LockedResources->Remove(ResourceName);
	}

	// Add lock activity and notify for non-temporary locks
	if (LockEvent.ResourceNames.Num() > 0)
	{
		FConcertSyncLockActivity LockActivity;
		LockActivity.EndpointId = InLockEndpointId;
		LockActivity.EventData.LockEventType = EConcertSyncLockEventType::Unlocked;
		LockActivity.EventData.ResourceNames = LockEvent.ResourceNames;
		LockActivity.EventSummary.SetTypedPayload(FConcertSyncLockActivitySummary::CreateSummaryForEvent(LockActivity.EventData));
		LockActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InLockEndpointId);
		AddLockActivity(LockActivity);

		LiveSession->GetSession().SendCustomEvent(LockEvent, LiveSession->GetSession().GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
	}

	return ReleasedLockCount == InResourceNames.Num();
}

void FConcertServerWorkspace::UnlockAllWorkspaceResources(const FGuid& InLockEndpointId)
{
	if (!LockedResources)
	{
		return;
	}

	FConcertResourceLockEvent LockEvent;
	for (auto It = LockedResources->CreateIterator(); It; ++It)
	{
		if (It->Value.EndpointId == InLockEndpointId)
		{
			LockEvent.ResourceNames.Add(It->Key);
			It.RemoveCurrent();
		}
	}
	// Notify lock state change
	if (LockEvent.ResourceNames.Num() > 0)
	{
		// Add lock activity
		{
			FConcertSyncLockActivity LockActivity;
			LockActivity.EndpointId = InLockEndpointId;
			LockActivity.EventData.LockEventType = EConcertSyncLockEventType::Unlocked;
			LockActivity.EventData.ResourceNames = LockEvent.ResourceNames;
			LockActivity.EventSummary.SetTypedPayload(FConcertSyncLockActivitySummary::CreateSummaryForEvent(LockActivity.EventData));
			LockActivity.bIgnored = ShouldIgnoreClientActivityOnRestore(InLockEndpointId);
			AddLockActivity(LockActivity);
		}

		LockEvent.ClientId = InLockEndpointId;
		LockEvent.LockType = EConcertResourceLockType::Unlock;
		LiveSession->GetSession().SendCustomEvent(LockEvent, LiveSession->GetSession().GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
	}
}

bool FConcertServerWorkspace::IsWorkspaceResourceLocked(const FName InResourceName, const FGuid& InLockEndpointId) const
{
	if (!LockedResources)
	{
		return true;
	}

	const FLockOwner* Owner = LockedResources->Find(InResourceName);
	return Owner && Owner->EndpointId == InLockEndpointId;
}

void FConcertServerWorkspace::SetEndpoint(const FGuid& InEndpointId, const FConcertSyncEndpointData& InEndpointData)
{
	// Update this endpoint and sync it
	if (LiveSession->GetSessionDatabase().SetEndpoint(InEndpointId, InEndpointData))
	{
		SyncCommandQueue->QueueCommand(LiveSyncEndpoints, [this, SyncEndpointId = InEndpointId](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			SendSyncEndpointEvent(InEndpointId, SyncEndpointId, InSyncCommandContext.GetNumRemainingCommands());
		});
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set endpoint '%ls' on live session '%ls': %ls", *InEndpointId.ToString(), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertServerWorkspace::SendSyncEndpointEvent(const FGuid& InTargetEndpointId, const FGuid& InSyncEndpointId, const int32 InNumRemainingSyncEvents) const
{
	FConcertWorkspaceSyncEndpointEvent SyncEvent;
	SyncEvent.NumRemainingSyncEvents = InNumRemainingSyncEvents;
	SyncEvent.Endpoint.EndpointId = InSyncEndpointId;
	if (LiveSession->GetSessionDatabase().GetEndpoint(InSyncEndpointId, SyncEvent.Endpoint.EndpointData))
	{
		LiveSession->GetSession().SendCustomEvent(SyncEvent, InTargetEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to get endpoint '%ls' from live session '%ls': %ls", *InSyncEndpointId.ToString(), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertServerWorkspace::AddConnectionActivity(const FConcertSyncConnectionActivity& InConnectionActivity)
{
	// Add the activity and sync it
	int64 ActivityId = 0;
	int64 EventId = 0;
	if (LiveSession->GetSessionDatabase().AddConnectionActivity(InConnectionActivity, ActivityId, EventId))
	{
		PostActivityAdded(ActivityId);
		SyncCommandQueue->QueueCommand(LiveSyncEndpoints, [this, SyncActivityId = ActivityId](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			SendSyncConnectionActivityEvent(InEndpointId, SyncActivityId, InSyncCommandContext.GetNumRemainingCommands());
		});
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set connection activity '%ls' on live session '%ls': %ls", *LexToString(ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertServerWorkspace::SendSyncConnectionActivityEvent(const FGuid& InTargetEndpointId, const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents) const
{
	FConcertSyncConnectionActivity SyncActivity;
	if (LiveSession->GetSessionDatabase().GetConnectionActivity(InSyncActivityId, SyncActivity))
	{
		FConcertWorkspaceSyncActivityEvent SyncEvent;
		SyncEvent.NumRemainingSyncEvents = InNumRemainingSyncEvents;
		SyncEvent.Activity.SetTypedPayload(SyncActivity);
		LiveSession->GetSession().SendCustomEvent(SyncEvent, InTargetEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to get connection activity '%ls' from live session '%ls': %ls", *LexToString(InSyncActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertServerWorkspace::AddLockActivity(const FConcertSyncLockActivity& InLockActivity)
{
	// Add the activity and sync it
	int64 ActivityId = 0;
	int64 EventId = 0;
	if (LiveSession->GetSessionDatabase().AddLockActivity(InLockActivity, ActivityId, EventId))
	{
		PostActivityAdded(ActivityId);
		SyncCommandQueue->QueueCommand(LiveSyncEndpoints, [this, SyncActivityId = ActivityId](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			SendSyncLockActivityEvent(InEndpointId, SyncActivityId, InSyncCommandContext.GetNumRemainingCommands());
		});
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set lock activity '%ls' on live session '%ls': %ls", *LexToString(ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertServerWorkspace::SendSyncLockActivityEvent(const FGuid& InTargetEndpointId, const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents) const
{
	FConcertSyncLockActivity SyncActivity;
	if (LiveSession->GetSessionDatabase().GetLockActivity(InSyncActivityId, SyncActivity))
	{
		FConcertWorkspaceSyncActivityEvent SyncEvent;
		SyncEvent.NumRemainingSyncEvents = InNumRemainingSyncEvents;
		SyncEvent.Activity.SetTypedPayload(SyncActivity);
		LiveSession->GetSession().SendCustomEvent(SyncEvent, InTargetEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to get lock activity '%ls' from live session '%ls': %ls", *LexToString(InSyncActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertServerWorkspace::AddTransactionActivity(const FConcertSyncTransactionActivity& InTransactionActivity)
{
	// Add the activity and sync it
	int64 ActivityId = 0;
	int64 EventId = 0;
	if (LiveSession->GetSessionDatabase().AddTransactionActivity(InTransactionActivity, ActivityId, EventId))
	{
		PostActivityAdded(ActivityId);
		SyncCommandQueue->QueueCommand([this, SyncActivityId = ActivityId](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			TOptional<FConcertWorkspaceSyncActivityEvent> Event = SyncTransactionActivityEvent(SyncActivityId, InSyncCommandContext.GetNumRemainingCommands());
			if (Event)
			{
				LiveSession->GetSession().SendCustomEvent(Event.GetValue(), LiveSyncEndpoints, EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
			}
		});
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set transaction activity '%ls' on live session '%ls': %ls", *LexToString(ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

FGuid FConcertServerWorkspace::HashEndpointIdAndActivityId(const FGuid& TargetEndpointId, FActivityID ActivityID)
{
	const int32 LowBits = static_cast<int32>(ActivityID);
	const int32 HighBits = (ActivityID >> 32);
	return FGuid(TargetEndpointId.A + LowBits, TargetEndpointId.B + HighBits, TargetEndpointId.C, TargetEndpointId.D);
}

TOptional<FConcertWorkspaceSyncActivityEvent> FConcertServerWorkspace::SyncTransactionActivityEvent(const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents, const bool InLiveOnly) const
{
	FConcertSyncTransactionActivity SyncActivity;
	if (LiveSession->GetSessionDatabase().GetActivity(InSyncActivityId, SyncActivity))
	{
		bool bMetaDataOnly = false;
		{
			FConcertSessionFilter SessionFilter;
			SessionFilter.bOnlyLiveData = InLiveOnly;
			bMetaDataOnly = !ConcertSyncSessionDatabaseFilterUtil::TransactionEventPassesFilter(SyncActivity.EventId, SessionFilter, LiveSession->GetSessionDatabase());
		}

		if (!LiveSession->GetSessionDatabase().GetTransactionEvent(SyncActivity.EventId, SyncActivity.EventData, bMetaDataOnly))
		{
			UE_LOGF(LogConcert, Error, "Failed to get transaction event '%ls' from live session '%ls': %ls", *LexToString(SyncActivity.EventId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
		}

		FConcertWorkspaceSyncActivityEvent SyncEvent;
		SyncEvent.NumRemainingSyncEvents = InNumRemainingSyncEvents;
		SyncEvent.Activity.SetTypedPayload(SyncActivity);
		return MoveTemp(SyncEvent);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to get transaction activity '%ls' from live session '%ls': %ls", *LexToString(InSyncActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
	return {};
}

void FConcertServerWorkspace::SendSyncTransactionActivityEvent(const FGuid& InTargetEndpointId, const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents, const bool InLiveOnly) const
{
	TOptional<FConcertWorkspaceSyncActivityEvent> SyncEvent = SyncTransactionActivityEvent(InSyncActivityId, InNumRemainingSyncEvents, InLiveOnly);
	if (SyncEvent)
	{
		LiveSession->GetSession().SendCustomEvent(SyncEvent.GetValue(), InTargetEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
}

void FConcertServerWorkspace::AddPackageActivity(const FConcertSyncActivity& InPackageActivityBasePart, const FConcertPackageInfo& PackageInfo, FConcertPackageDataStream& InPackageDataStream)
{
	SCOPED_CONCERT_TRACE(FConcertServerWorkspace_AddPackageActivity);
	// Add the activity and sync it
	int64 ActivityId = 0;
	int64 EventId = 0;

	if (LiveSession->GetSessionDatabase().AddPackageActivity(InPackageActivityBasePart, PackageInfo, InPackageDataStream, ActivityId, EventId))
	{
		PostActivityAdded(ActivityId);
		TOptional<FConcertWorkspaceSyncActivityEvent> SyncEvent = MakeSyncActivityEvent(ActivityId);
		if(SyncEvent)
		{
			SyncCommandQueue->QueueCommand(
				LiveSyncEndpoints,
				[this,Event = MoveTemp(SyncEvent)](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId) mutable
			{
				Event->NumRemainingSyncEvents = InSyncCommandContext.GetNumRemainingCommands();
				SendSyncPackageActivityEvent(Event.GetValue(), InEndpointId);
			});
		}
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set package activity '%ls' on live session '%ls': %ls", *LexToString(ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

TOptional<FConcertWorkspaceSyncActivityEvent> FConcertServerWorkspace::MakeSyncActivityEvent(const int64 InSyncActivityId, bool bInHeadOnly) const
{
	SCOPED_CONCERT_TRACE(FConcertServerWorkspace_MakeSyncActivityEvent);
	FConcertSyncPackageActivity SyncActivity;
	if (LiveSession->GetSessionDatabase().GetActivity(InSyncActivityId, SyncActivity))
	{
		bool bMetaDataOnly = false;
		{
			FConcertSessionFilter SessionFilter;
			SessionFilter.bOnlyLiveData = bInHeadOnly;
			bMetaDataOnly = !ConcertSyncSessionDatabaseFilterUtil::PackageEventPassesFilter(SyncActivity.EventId, SessionFilter, LiveSession->GetSessionDatabase());
		}

		if (!bMetaDataOnly) // Package data is needed.
		{
			// Callback function used to fill the package event (containing the package data, which can be small or large).
			EConcertSessionResponseCode FillPackageEventResponseCode = EConcertSessionResponseCode::Success;
			auto FillPackageEventFn = [this, &SyncActivity, &FillPackageEventResponseCode](FConcertSyncPackageEventData& PackageEventSrc)
			{
				// May embed the package data directly or link it if its too big and file sharing is enabled.
				FillPackageEventResponseCode = FillPackageEvent(PackageEventSrc, SyncActivity.EventData);
			};

			if (!LiveSession->GetSessionDatabase().GetPackageEvent(SyncActivity.EventId, FillPackageEventFn) || FillPackageEventResponseCode == EConcertSessionResponseCode::Failed)
			{
				UE_LOGF(LogConcert, Error,
					   "Failed to get package event '%ls' from live session '%ls': %ls",
					   *LexToString(SyncActivity.EventId),
					   *LiveSession->GetSession().GetName(),
					   *LiveSession->GetSessionDatabase().GetLastError());
			}
		}
		else if (!LiveSession->GetSessionDatabase().GetPackageEventMetaData(SyncActivity.EventId, SyncActivity.EventData.PackageRevision, SyncActivity.EventData.Package.Info))
		{
			UE_LOGF(LogConcert, Error, "Failed to get package event '%ls' from live session '%ls': %ls",
				   *LexToString(SyncActivity.EventId),
				   *LiveSession->GetSession().GetName(),
				   *LiveSession->GetSessionDatabase().GetLastError());
		}

		FConcertWorkspaceSyncActivityEvent SyncEvent;
		// If SyncActivity was already compressed then there is no need to compress it again.
		SyncEvent.Activity.SetTypedPayload(SyncActivity,EConcertPayloadCompressionType::None);
		return MoveTemp(SyncEvent);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to get package activity '%ls' from live session '%ls': %ls",
			   *LexToString(InSyncActivityId),
			   *LiveSession->GetSession().GetName(),
			   *LiveSession->GetSessionDatabase().GetLastError());
	}
	return {};//FConcertWorkspaceSyncActivityEvent;
}

void FConcertServerWorkspace::SendSyncPackageActivityEvent(const FConcertWorkspaceSyncActivityEvent& SyncEvent, const FGuid& InTargetEndpointId) const
{
	SCOPED_CONCERT_TRACE(FConcertServerWorkspace_SendSyncPackageActivityEvent);

	if (UE::ConcertSyncCore::ConcertPackageEvents::OnLocalBeginSendPackage().IsBound())
	{
		FConcertSyncPackageActivity PackageActivity;
		SyncEvent.Activity.GetTypedPayload(PackageActivity);
		const FGuid TransmissionId = HashEndpointIdAndActivityId(InTargetEndpointId, PackageActivity.ActivityId);
		UE::ConcertSyncCore::ConcertPackageEvents::FConcertBeginSendPackageParams PreTransmitEvent{ { TransmissionId, PackageActivity.EventData.Package.Info, InTargetEndpointId } };
		// This loads the file even though we only want the file size... could be improved
		LiveSession->GetSessionDatabase().GetPackageEvent(PackageActivity.EventId, [&PreTransmitEvent](const FConcertSyncPackageEventData& Data)
		{
			PreTransmitEvent.PackageNumBytes = static_cast<uint64>(Data.PackageDataStream.DataSize);
		});
		
		if (PackageActivity.EventData.Package.HasPackageData())
		{
			UE::ConcertSyncCore::ConcertPackageEvents::OnLocalBeginSendPackage().Broadcast(PreTransmitEvent);
		}
	}
	
	LiveSession->GetSession().SendCustomEvent(SyncEvent, InTargetEndpointId, EConcertMessageFlags::ReliableOrdered);
}

void FConcertServerWorkspace::OnAddReplicationActivity(const int64 ActivityId, const bool bSuccess)
{
	if (bSuccess)
	{
		PostActivityAdded(ActivityId);
		SyncCommandQueue->QueueCommand(LiveSyncEndpoints, [this, SyncActivityId = ActivityId](const FConcertServerSyncCommandQueue::FSyncCommandContext& InSyncCommandContext, const FGuid& InEndpointId)
		{
			SendSyncReplicationActivityEvent(InEndpointId, SyncActivityId, InSyncCommandContext.GetNumRemainingCommands());
		});
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set replication activity '%ls' on live session '%ls': %ls", *LexToString(ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}	
}

void FConcertServerWorkspace::SendSyncReplicationActivityEvent(const FGuid& InTargetEndpointId, const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents)
{
	FConcertSyncReplicationActivity SyncActivity;
	if (LiveSession->GetSessionDatabase().GetReplicationActivity(InSyncActivityId, SyncActivity))
	{
		FConcertWorkspaceSyncActivityEvent SyncEvent;
		SyncEvent.NumRemainingSyncEvents = InNumRemainingSyncEvents;
		SyncEvent.Activity.SetTypedPayload(SyncActivity);
		LiveSession->GetSession().SendCustomEvent(SyncEvent, InTargetEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to get replication activity '%ls' from live session '%ls': %ls", *LexToString(InSyncActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertServerWorkspace::PostActivityAdded(const int64 InActivityId)
{
	FConcertSyncActivity Activity;
	if (LiveSession->GetSessionDatabase().GetActivity(InActivityId, Activity))
	{
		LiveSession->GetSessionDatabase().OnActivityProduced().Broadcast(Activity);
		
		FConcertSyncEndpointData EndpointData;
		if (LiveSession->GetSessionDatabase().GetEndpoint(Activity.EndpointId, EndpointData))
		{
			FStructOnScope ActivitySummary;
			if (Activity.EventSummary.GetPayload(ActivitySummary))
			{
				check(ActivitySummary.GetStruct()->IsChildOf(FConcertSyncActivitySummary::StaticStruct()));
				const FConcertSyncActivitySummary* ActivitySummaryPtr = (FConcertSyncActivitySummary*)ActivitySummary.GetStructMemory();
				UE_LOGF(LogConcert, Display, "Endpoint '%ls' produced activity '%ls': %ls", *Activity.EndpointId.ToString(), *LexToString(Activity.ActivityId), *ActivitySummaryPtr->ToDisplayText(FText::AsCultureInvariant(EndpointData.ClientInfo.DisplayName)).ToString());
			}
		}
	}
}

bool FConcertServerWorkspace::IsFileSharingEnabled() const
{
	return FileSharingService && FileSharingService->IsEnabled() && EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableFileSharing);
}

bool FConcertServerWorkspace::CanExchangePackageDataAsByteArray(int64 PackageDataSize) const
{
	if (IsFileSharingEnabled())
	{
		return FConcertPackage::ShouldEmbedPackageDataAsByteArray(PackageDataSize); // Test the package data size against a preferred limit.
	}

	return FConcertPackage::CanEmbedPackageDataAsByteArray(PackageDataSize); // The the package data size against the maximum permitted.
}

EConcertSessionResponseCode FConcertServerWorkspace::FillPackageEvent(FConcertSyncPackageEventData& InPackageEvent, FConcertSyncPackageEvent& OutPackageEvent) const
{
	SCOPED_CONCERT_TRACE(FConcertSaveWorkspace_FillPackageEvent);
	EConcertSessionResponseCode FillPackageEventResponseCode = EConcertSessionResponseCode::Success;

	// Fill the meta data.
	OutPackageEvent.PackageRevision = InPackageEvent.MetaData.PackageRevision;
	OutPackageEvent.Package.Info = InPackageEvent.MetaData.PackageInfo;

	// Fill the package data.
	if (CanExchangePackageDataAsByteArray(InPackageEvent.PackageDataStream.DataSize))
	{
		// Store the package data directly in the package event.
		OutPackageEvent.Package.PackageData.Bytes.AddUninitialized(InPackageEvent.PackageDataStream.DataSize);
		InPackageEvent.PackageDataStream.DataAr->Serialize(OutPackageEvent.Package.PackageData.Bytes.GetData(), InPackageEvent.PackageDataStream.DataSize);
	}
	else if (IsFileSharingEnabled())
	{
		// Share a link to the file.
		FillPackageEventResponseCode = FileSharingService->Publish(*InPackageEvent.PackageDataStream.DataAr, InPackageEvent.PackageDataStream.DataSize, OutPackageEvent.Package.FileId) ? EConcertSessionResponseCode::Success : EConcertSessionResponseCode::Failed;
	}
	else
	{
		// The package data is too big.
		FillPackageEventResponseCode = EConcertSessionResponseCode::Failed;
	}

	return FillPackageEventResponseCode;
}

TSharedPtr<FArchive> FConcertServerWorkspace::GetPackageDataStream(const FConcertPackage& Package)
{
	if (!Package.FileId.IsEmpty() && ensure(FileSharingService))
	{
		return FileSharingService->CreateReader(Package.FileId);
	}
	else if (Package.PackageData.Bytes.Num() > 0)
	{
		return MakeShared<FMemoryReader>(Package.PackageData.Bytes); // Package data is embedded in the activity object.		
	}
	
	return nullptr;
}

