// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertWorkspaceMessages.h"
#include "IConcertSessionHandler.h"
#include "Misc/Optional.h"
#include "Replication/IReplicationWorkspace.h"
#include "Replication/ReplicationWorkspace.h"

class IConcertFileSharingService;
class IConcertMessage;
class IConcertServerSession;
class FConcertSyncServerLiveSession;
class FConcertServerSyncCommandQueue;
class FConcertServerDataStore;

struct FConcertMessageContext;
struct FConcertEndpointContext;
struct FConcertSyncReplicationActivity;
struct FConcertSyncReplicationPayload_LeaveReplication;
struct FConcertTransactionSnapshotEvent;
struct FConcertTransactionFinalizedEvent;

enum class EConcertLockFlags : uint8
{
	None		= 0,
	Explicit	= 1 << 0,
	Force		= 1 << 1,
	Temporary	= 1 << 2,
};
ENUM_CLASS_FLAGS(EConcertLockFlags);

class FConcertServerWorkspace
{
public:
	
	explicit FConcertServerWorkspace(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession, TSharedPtr<IConcertFileSharingService> InFileSharingService);
	~FConcertServerWorkspace();

	/** @return Gets the portion of the workspace that manages replication data. */
	UE::ConcertSyncServer::Replication::IReplicationWorkspace& GetReplicationWorkspace() { return ReplicationWorkspace; }

private:
	
	/** Creates an ID from the given data through an simple hash function. The ID only depends on the input, allowing us to reconstruct IDs later in time. */
	static FGuid HashEndpointIdAndActivityId(const FGuid& TargetEndpointId, FActivityID ActivityID);
	
	/**
	 * Get the sync event for a transaction activity in the session database.
	 *
	 * @param InSyncActivityId			The ID of the activity to send the sync event for.
	 * @param InNumRemainingSyncEvents	The number of items left in the sync queue.
	 * @param InLiveOnly				True if the bulk of the transaction data should only be sent if this transaction is live.
	 * Returns the transaction activity event for the given activity id.
	 */
	TOptional<FConcertWorkspaceSyncActivityEvent> SyncTransactionActivityEvent(const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents, const bool InLiveOnly = true) const;

	/**
	 * Get the sync event for a package activity in the session database.
	 *
	 * @param InSyncActivityId			The ID of the activity to send the sync event for.
	 * @param InHeadOnly				True if we are only interested in the head revision
	 * Returns the package activity event for the given activity id.
	 */
	TOptional<FConcertWorkspaceSyncActivityEvent> MakeSyncActivityEvent(const int64 InSyncActivityId, const bool bInHeadOnly = true) const;

	/** Bind the workspace to this session. */
	void BindSession(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession);

	/** Unbind the workspace to its bound session. */
	void UnbindSession();

	/** */
	void HandleTick(IConcertServerSession& InSession, float InDeltaTime);

	/** */
	void HandleSessionClientChanged(IConcertServerSession& InSession, EConcertClientStatus InClientStatus, const FConcertSessionClientInfo& InClientInfo);

	/** */
	void HandleSessionAcknowledgementReceived(const FConcertEndpointContext& LocalEndpoint, const FConcertEndpointContext& RemoteEndpoint, const TSharedRef<IConcertMessage>& AckedMessage, const FConcertMessageContext& MessageContext) const;

	/** */
	void HandleSyncRequestedEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncRequestedEvent& Event);

	/** */
	void HandlePackageTransmissionStartEvent(const FConcertSessionContext& Context, const FConcertPackageTransmissionStartEvent& Event);
	
	/** */
	void HandlePackageUpdateEvent(const FConcertSessionContext& Context, const FConcertPackageUpdateEvent& Event);

	/** */
	void HandleTransactionFinalizedEvent(const FConcertSessionContext& InEventContext, const FConcertTransactionFinalizedEvent& InEvent);

	/** */
	void HandleTransactionSnapshotEvent(const FConcertSessionContext& InEventContext, const FConcertTransactionSnapshotEvent& InEvent);

	/** */
	void HandlePlaySessionEvent(const FConcertSessionContext& Context, const FConcertPlaySessionEvent& Event);

	/** */
	EConcertSessionResponseCode HandleSyncEventRequest(const FConcertSessionContext& Context, const FConcertSyncEventRequest& Request, FConcertSyncEventResponse& Response);

	/** */
	EConcertSessionResponseCode HandleResourceLockRequest(const FConcertSessionContext& Context, const FConcertResourceLockRequest& Request, FConcertResourceLockResponse& Response);

	/** */
	void HandleIgnoredActivityStateChanged(const FConcertSessionContext& Context, const FConcertIgnoreActivityStateChangedEvent& Event);

	/** Invoked when the client corresponding to the specified endpoint begins to "Play" in a mode such as PIE or SIE. */
	void HandleBeginPlaySession(const FName InPlayPackageName, const FGuid& InEndpointId, bool bIsSimulating);

	/** Invoked when the client corresponding to the specified endpoint exits a "Play" mode such as PIE or SIE. */
	void HandleEndPlaySession(const FName InPlayPackageName, const FGuid& InEndpointId);

	/** Invoked when the client corresponding to specified endpoint toggles between PIE and SIE play mode. */
	void HandleSwitchPlaySession(const FName InPlayPackageName, const FGuid& InEndpointId);

	/** Invoked when the cient corresponding to the specified endpoint exits a "Play" mode such as PIE or SIE. */
	void HandleEndPlaySessions(const FGuid& InEndpointId);

	/** A special event sent from the client to enable / disable additional logging on the server. */
	void HandleServerLoggingEvent(const FConcertSessionContext& Context, const FConcertServerLogging& Event);

	/** Returns the package name being played (PIE/SIE) by the specified client endpoint if that endpoint is in such play mode, otherwise, returns an empty name. */
	FName FindPlaySession(const FGuid& InEndpointId);

	/**
	 * Attempt to lock the given resource to the given endpoint.
	 * @note Passing force will always assign the lock to the given endpoint, even if currently locked by another.
	 * @return True if the resource was locked (or already locked by the given endpoint), false otherwise.
	 */
	bool LockWorkspaceResource(const FName InResourceName, const FGuid& InLockEndpointId, EConcertLockFlags InLockFlags);

	/**
	 * Attempt to lock a list of resources to the given endpoint.
	 * @param InResourceNames The list of resource to lock
	 * @param InLockEndpointId The client id trying to acquire the lock
	 * @param InExplicit mark the lock as explicit
	 * @param InForce steal locks if true
	 * @param OutFailedResources Pointer to an array to gather resources on which acquiring the lock failed.
	 * @return true if the lock was successfully acquired on all InResourceNames
	 */
	bool LockWorkspaceResources(const TArray<FName>& InResourceNames, const FGuid& InLockEndpointId, EConcertLockFlags InLockFlags, TMap<FName, FGuid>* OutFailedRessources = nullptr);

	/**
	 * Attempt to unlock the given resource from the given endpoint.
	 * @note Passing force will always clear, even if currently locked by another endpoint.
	 * @return True if the resource was unlocked, false otherwise.
	 */
	bool UnlockWorkspaceResource(const FName InResourceName, const FGuid& InLockEndpointId, EConcertLockFlags InLockFlags);

	/**
	 * Attempt to unlock a list of resources from the given endpoint.
	 * @param InResourceNames The list of resource to unlock
	 * @param InLockEndpointId The client id trying to releasing the lock
	 * @param InExplicit mark the unlock as explicit, implicit unlock won't unlock explicit lock
	 * @param InForce release locks even of not owned if true
	 * @param OutFailedResources Pointer to an array to gather resources on which releasing the lock failed.
	 * @return true if the lock was successfully released on all InResourceNames
	 */
	bool UnlockWorkspaceResources(const TArray<FName>& InResourceNames, const FGuid& InLockEndpointId, EConcertLockFlags InLockFlags, TMap<FName, FGuid>* OutFailedRessources = nullptr);

	/**
	 * Unlock all resource locks held by a client.
	 * @param InLockEndpointId the client endpoint id releasing the lock on resources.
	 */
	void UnlockAllWorkspaceResources(const FGuid& InLockEndpointId);

	/**
	 * Check to see if the given resource is locked by the given endpoint.
	 */
	bool IsWorkspaceResourceLocked(const FName InResourceName, const FGuid& InLockEndpointId) const;

	/**
	 * Set an endpoint in the session database, creating or replacing it, and sync the result back to all clients.
	 *
	 * @param InEndpointId				The ID of the endpoint to set.
	 * @param InEndpointData			The endpoint data to set.
	 */
	void SetEndpoint(const FGuid& InEndpointId, const FConcertSyncEndpointData& InEndpointData);

	/**
	 * Send a sync event for an endpoint in the session database.
	 *
	 * @param InTargetEndpointId		The ID of the endpoint to send the sync event to.
	 * @param InSyncEndpointId			The ID of the endpoint to send the sync event for.
	 * @param InNumRemainingSyncEvents	The number of items left in the sync queue.
	 */
	void SendSyncEndpointEvent(const FGuid& InTargetEndpointId, const FGuid& InSyncEndpointId, const int32 InNumRemainingSyncEvents) const;

	/**
	 * Add a new connection activity to the session database, assigning it both an activity and connection event ID, and sync the result back to all clients.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InConnectionActivity		The connection activity to add (the ActivityId, EventTime, EventType, and EventId members are ignored).
	 */
	void AddConnectionActivity(const FConcertSyncConnectionActivity& InConnectionActivity);

	/**
	 * Send a sync event for a connection activity in the session database.
	 *
	 * @param InTargetEndpointId		The ID of the endpoint to send the sync event to.
	 * @param InSyncActivityId			The ID of the activity to send the sync event for.
	 * @param InNumRemainingSyncEvents	The number of items left in the sync queue.
	 */
	void SendSyncConnectionActivityEvent(const FGuid& InTargetEndpointId, const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents) const;

	/**
	 * Add a new lock activity to the session database, assigning it both an activity and lock event ID, and sync the result back to all clients.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InLockActivity			The lock activity to add (the ActivityId, EventTime, EventType, and EventId members are ignored).
	 */
	void AddLockActivity(const FConcertSyncLockActivity& InLockActivity);

	/**
	 * Send a sync event for a lock activity in the session database.
	 *
	 * @param InTargetEndpointId		The ID of the endpoint to send the sync event to.
	 * @param InSyncActivityId			The ID of the activity to send the sync event for.
	 * @param InNumRemainingSyncEvents	The number of items left in the sync queue.
	 */
	void SendSyncLockActivityEvent(const FGuid& InTargetEndpointId, const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents) const;

	/**
	 * Add a new transaction activity to the session database, assigning it both an activity and transaction event ID, and sync the result back to all clients.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InTransactionActivity		The transaction activity to add (the ActivityId, EventTime, EventType, and EventId members are ignored).
	 */
	void AddTransactionActivity(const FConcertSyncTransactionActivity& InTransactionActivity);

	/**
	 * Send a sync event for a transaction activity in the session database.
	 *
	 * @param InTargetEndpointId		The ID of the endpoint to send the sync event to.
	 * @param InSyncActivityId			The ID of the activity to send the sync event for.
	 * @param InNumRemainingSyncEvents	The number of items left in the sync queue.
	 * @param InLiveOnly				True if the bulk of the transaction data should only be sent if this transaction is live.
	 */
	void SendSyncTransactionActivityEvent(const FGuid& InTargetEndpointId, const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents, const bool InLiveOnly = true) const;

	/**
	 * Add a new package activity to the session database, assigning it both an activity and package event ID, and sync the result back to all clients.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InPackageActivity			The package activity to add (the ActivityId, EventTime, EventType, EventId, and PackageRevision members are ignored).
	 */
	void AddPackageActivity(const FConcertSyncActivity& InPackageActivityBasePart, const FConcertPackageInfo& PackageInfo, FConcertPackageDataStream& InPackageDataStream);

	/**
	 * Send a sync event for a package activity in the session database.
	 *
	 * @param SyncEvent				The ID of the endpoint to send the sync event to.
	 * @param InSyncActivityId		The ID of the activity to send the sync event for.
	 */
	void SendSyncPackageActivityEvent(const FConcertWorkspaceSyncActivityEvent& SyncEvent, const FGuid& InTargetEndpointId) const;

	/**
	 * Called when the replication workspace attempted to add a FConcertSyncReplicationActivity to the database.
	 * This syncs the result back to all clients.
	 *
	 * @param ActivityId	The identifier of the activity that was attempted to be added
	 * @param bSuccess		Whether the activity was actually added to the database (i.e. false means the attempt failed).
	 */
	void OnAddReplicationActivity(const int64 ActivityId, const bool bSuccess);
	
	/**
	 * Send a sync event for a replication activity in the session database.
	 *
	 * @param InTargetEndpointId		The ID of the endpoint to send the sync event to.
	 * @param InSyncActivityId			The ID of the activity to send the sync event for.
	 * @param InNumRemainingSyncEvents	The number of items left in the sync queue.
	 */
	void SendSyncReplicationActivityEvent(const FGuid& InTargetEndpointId, const int64 InSyncActivityId, const int32 InNumRemainingSyncEvents);
	
	/**
	 * Called after any activity is added to the session database.
	 *
	 * @param InActivityId				The ID of the activity that was added.
	 */
	void PostActivityAdded(const int64 InActivityId);

	/**
	 * Check if a given client was configured to prevent restoring the activities being recorded.
	 * @param ClientEndpoint The client for which the check must be done.
	 * @return True if the activity should be flagged as 'ignored on restore'.
	 */
	bool ShouldIgnoreClientActivityOnRestore(const FGuid& ClientEndpoint) const { return IgnoredActivityClients.Contains(ClientEndpoint); }

	/**
	 * Returns true if the package data is small enough to be exchanged using a single TArray<> data structure. If the package data is
	 * too big according to a configuration (or a hard limit) the package data will be exchanged through other means like file stream
	 * or file sharing.
	 */
	bool CanExchangePackageDataAsByteArray(int64 PackageDataSize) const;

	/** Returns true if the file sharing service is enabled. */
	bool IsFileSharingEnabled() const;
		
	/**
	 * Write all package event fields from the event and decide if the package data should be directly embedded in the event or linked to a shared file.
	 */
	EConcertSessionResponseCode FillPackageEvent(FConcertSyncPackageEventData& InPackageEventSrc, FConcertSyncPackageEvent& OutPackageEvent) const;

	/**
	 * Open a stream on the package data, depending if the package data was directly embedded in the object (in memory) or linked as a shared file.
	 */
	TSharedPtr<FArchive> GetPackageDataStream(const FConcertPackage& Package);

	/** Live session tracked by this workspace */
	TSharedPtr<FConcertSyncServerLiveSession> LiveSession;

	/** Array of endpoints that are subscribed to live-sync (server automatically pushes updates) */
	TArray<FGuid> LiveSyncEndpoints;

	/** Array of endpoints that are currently undergoing a manual sync (client explicitly request data) */
	TArray<FGuid> ManualSyncEndpoints;

	/** */
	TSharedPtr<FConcertServerSyncCommandQueue> SyncCommandQueue;

	/** Contains the play state (PIE/SIE) of a client endpoint. */
	struct FPlaySessionInfo
	{
		FGuid EndpointId;
		bool bIsSimulating;
		bool operator==(const FPlaySessionInfo& Other) const { return EndpointId == Other.EndpointId && bIsSimulating == Other.bIsSimulating; }
	};

	/** Tracks endpoints that are in a play session (package name -> {endpoint IDs, bSimulating}) */
	TMap<FName, TArray<FPlaySessionInfo>> ActivePlaySessions;

	/** Tracks locked transaction resources (resource ID -> Lock owner) */
	struct FLockOwner
	{
		FGuid EndpointId;
		bool bExplicit = false;
		bool bTemporary = false;
	};
	typedef TMap<FName, FLockOwner> FLockedResources;
	TUniquePtr<FLockedResources> LockedResources;

	/** The data store shared by all clients connected to the server tracked by this workspace. */
	TUniquePtr<FConcertServerDataStore> DataStore;

	/** Contains the list of clients for which activities should be marked as 'ignore on restore'. @see FConcertIgnoreActivityStateChangedEvent. */
	TSet<FGuid> IgnoredActivityClients;

	/** Optional side channel to exchange large blobs (package data) with the client in a scalable way (ex. the request/response transport layer is not designed and doesn't support exchanging 3GB packages). */
	TUniquePtr<IConcertFileSharingService> FileSharingService;

	/** Manages replication data for the database. */
	UE::ConcertSyncServer::FReplicationWorkspace ReplicationWorkspace;
};
