// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncServer.h"

#include "IConcertServer.h"
#include "IConcertSession.h"
#include "ConcertServerWorkspace.h"
#include "ConcertServerSequencerManager.h"
#include "ConcertSyncServerLiveSession.h"
#include "ConcertSyncServerArchivedSession.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertLogGlobal.h"
#include "Replication/ConcertServerReplicationManager.h"
#include "Replication/Messages/ReplicationActivity.h"

#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "StructSerializer.h"
#include "StructDeserializer.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "Backends/JsonStructDeserializerBackend.h"

#define LOCTEXT_NAMESPACE "ConcertSyncServer"

namespace ConcertSyncServerUtils
{

const TCHAR SessionInfoFilename[] = TEXT("SessionInfo.json");

/** Deduce the creation time of a session from the creation time of its containing folder. */
FDateTime GetSessionCreationTime(const FString& InSessionRoot)
{
	IFileManager& FileManager = IFileManager::Get();

	FFileStatData FileStatData = FileManager.GetStatData(*InSessionRoot);
	check(FileStatData.bIsValid && FileStatData.CreationTime != FDateTime::MinValue());
	return FileStatData.CreationTime;
}

/** Write the session info file to a session root directory */
bool WriteSessionInfoToDirectory(const FString& InPath, const FConcertSessionInfo& InSessionInfo)
{
	const FString SessionInfoPathname = InPath / SessionInfoFilename;

	TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*SessionInfoPathname));
	if (FileWriter)
	{
		FJsonStructSerializerBackend Backend(*FileWriter, EStructSerializerBackendFlags::Default);

		FStructSerializer::Serialize(InSessionInfo, Backend);

		FileWriter->Close();
		return !FileWriter->IsError();
	}
	
	return false;
}

/** Read the session info file from a session root directory */
bool ReadSessionInfoFromDirectory(const FString& InPath, FConcertSessionInfo& OutSessionInfo)
{
	const FString SessionInfoPathname = InPath / SessionInfoFilename;

	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*SessionInfoPathname));
	if (FileReader)
	{
		FJsonStructDeserializerBackend Backend(*FileReader);

		FStructDeserializer::Deserialize(OutSessionInfo, Backend);

		FileReader->Close();
		return !FileReader->IsError();
	}

	return false;
}

bool MigrateSessionData(const FConcertSyncSessionDatabase& InSourceDatabase, const FString& InDestSessionPath, const FConcertSessionFilter& InDestSessionFilter)
{
	check(InSourceDatabase.IsValid());

	FConcertSyncSessionDatabase DestDatabase;
	if (!DestDatabase.Open(InDestSessionPath))
	{
		return false;
	}

	bool bResult = true;
#define MIGRATE_SET_ERROR_RESULT_AND_RETURN(MSG, ...)	\
	UE_LOG(LogConcert, Error, TEXT(MSG), __VA_ARGS__);	\
	bResult = false;									\
	return false

	bResult &= InSourceDatabase.EnumerateEndpoints([&bResult, &DestDatabase](FConcertSyncEndpointIdAndData&& InEndpointData)
	{
		if (!DestDatabase.SetEndpoint(InEndpointData.EndpointId, InEndpointData.EndpointData))
		{
			MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to set endpoint '%s' on database at '%s': %s", *InEndpointData.EndpointId.ToString(), *DestDatabase.GetFilename(), *DestDatabase.GetLastError());
		}
		return true;
	});

	if (bResult)
	{
		bResult &= InSourceDatabase.EnumerateActivityIdsAndEventTypes([&bResult, &InSourceDatabase, &InDestSessionFilter, &DestDatabase](const int64 InActivityId, const EConcertSyncActivityEventType InEventType)
		{
			if (!InDestSessionFilter.ActivityIdPassesFilter(InActivityId))
			{
				return true;
			}

			static_assert(static_cast<int32>(EConcertSyncActivityEventType::Count) == 6, "If you added an EConcertSyncActivityEventType entry, update this switch");
			switch (InEventType)
			{
			case EConcertSyncActivityEventType::Connection:
			{
				FConcertSyncConnectionActivity ConnectionActivity;
				if (!InSourceDatabase.GetConnectionActivity(InActivityId, ConnectionActivity))
				{
					MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to get connection activity '%s' from database at '%s': %s", *LexToString(InActivityId), *InSourceDatabase.GetFilename(), *InSourceDatabase.GetLastError());
				}
				if (InDestSessionFilter.bIncludeIgnoredActivities || !ConnectionActivity.bIgnored)
				{
					if (!DestDatabase.SetConnectionActivity(ConnectionActivity))
					{
						MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to set connection activity '%s' on database at '%s': %s", *LexToString(InActivityId), *DestDatabase.GetFilename(), *DestDatabase.GetLastError());
					}
				}
			}
			break;

			case EConcertSyncActivityEventType::Lock:
			{
				FConcertSyncLockActivity LockActivity;
				if (!InSourceDatabase.GetLockActivity(InActivityId, LockActivity))
				{
					MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to get lock activity '%s' from database at '%s': %s", *LexToString(InActivityId), *InSourceDatabase.GetFilename(), *InSourceDatabase.GetLastError());
				}
				if (InDestSessionFilter.bIncludeIgnoredActivities || !LockActivity.bIgnored)
				{
					if (!DestDatabase.SetLockActivity(LockActivity))
					{
						MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to set lock activity '%s' on database at '%s': %s", *LexToString(InActivityId), *DestDatabase.GetFilename(), *DestDatabase.GetLastError());
					}
				}
			}
			break;

			case EConcertSyncActivityEventType::Transaction:
			{
				FConcertSyncTransactionActivity TransactionActivity;
				if (!InSourceDatabase.GetActivity(InActivityId, TransactionActivity))
				{
					MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to get transaction activity '%s' from database at '%s': %s", *LexToString(InActivityId), *InSourceDatabase.GetFilename(), *InSourceDatabase.GetLastError());
				}
				if ((InDestSessionFilter.bIncludeIgnoredActivities || !TransactionActivity.bIgnored) && ConcertSyncSessionDatabaseFilterUtil::TransactionEventPassesFilter(TransactionActivity.EventId, InDestSessionFilter, InSourceDatabase))
				{
					if (!InSourceDatabase.GetTransactionEvent(TransactionActivity.EventId, TransactionActivity.EventData, InDestSessionFilter.bMetaDataOnly))
					{
						MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to get transaction event '%s' from database at '%s': %s", *LexToString(TransactionActivity.EventId), *InSourceDatabase.GetFilename(), *InSourceDatabase.GetLastError());
					}
					if (!DestDatabase.SetTransactionActivity(TransactionActivity, InDestSessionFilter.bMetaDataOnly))
					{
						MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to set transaction activity '%s' on database at '%s': %s", *LexToString(InActivityId), *DestDatabase.GetFilename(), *DestDatabase.GetLastError());
					}
				}
			}
			break;

			case EConcertSyncActivityEventType::Package:
			{
				FConcertSyncActivity PackageActivityBasePart;
				if (!InSourceDatabase.GetActivity(InActivityId, PackageActivityBasePart)) // This only fill up the part that is common to all activities.
				{
					MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to get package activity '%s' from database at '%s': %s", *LexToString(InActivityId), *InSourceDatabase.GetFilename(), *InSourceDatabase.GetLastError());
				}
				if ((InDestSessionFilter.bIncludeIgnoredActivities || !PackageActivityBasePart.bIgnored) && ConcertSyncSessionDatabaseFilterUtil::PackageEventPassesFilter(PackageActivityBasePart.EventId, InDestSessionFilter, InSourceDatabase))
				{
					if (InDestSessionFilter.bMetaDataOnly)
					{
						FConcertSyncPackageEventData PackageActivityEventPart;
						if (!InSourceDatabase.GetPackageEventMetaData(PackageActivityBasePart.EventId, PackageActivityEventPart.MetaData.PackageRevision, PackageActivityEventPart.MetaData.PackageInfo))
						{
							check(PackageActivityEventPart.PackageDataStream.DataAr == nullptr);
							MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to get package event '%s' from database at '%s': %s", *LexToString(PackageActivityBasePart.EventId), *InSourceDatabase.GetFilename(), *InSourceDatabase.GetLastError());
						}
						// Merge the base part with the event part to reconstruct the package activity.
						if (!DestDatabase.SetPackageActivity(PackageActivityBasePart, PackageActivityEventPart, InDestSessionFilter.bMetaDataOnly))
						{
							MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to set package activity '%s' on database at '%s': %s", *LexToString(InActivityId), *DestDatabase.GetFilename(), *DestDatabase.GetLastError());
						}
					}
					else
					{
						// Pull the package event required to fill the package specific part of the activity.
						bool bSuccess = InSourceDatabase.GetPackageEvent(PackageActivityBasePart.EventId, [&DestDatabase, &PackageActivityBasePart, &InDestSessionFilter, InActivityId](FConcertSyncPackageEventData& PackageActivityEventPart)
						{
							// Merge the common activity part with the package event to reconstruct and migrate a package activity.
							if (!DestDatabase.SetPackageActivity(PackageActivityBasePart, PackageActivityEventPart, InDestSessionFilter.bMetaDataOnly))
							{
								UE_LOGF(LogConcert, Error, "Failed to set package activity '%ls' on database at '%ls': %ls", *LexToString(InActivityId), *DestDatabase.GetFilename(), *DestDatabase.GetLastError());
							}
						});

						if (!bSuccess)
						{
							MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to get package event '%s' from database at '%s': %s", *LexToString(PackageActivityBasePart.EventId), *InSourceDatabase.GetFilename(), *InSourceDatabase.GetLastError());
						}
					}
				}
			}
			break;

			case EConcertSyncActivityEventType::Replication:
			{
				FConcertSyncReplicationActivity ReplicationActivity;
				if (!InSourceDatabase.GetReplicationActivity(InActivityId, ReplicationActivity))
				{
					MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to get replication activity '%s' from database at '%s': %s", *LexToString(InActivityId), *InSourceDatabase.GetFilename(), *InSourceDatabase.GetLastError());
				}
				if (InDestSessionFilter.bIncludeIgnoredActivities || !ReplicationActivity.bIgnored)
				{
					if (!DestDatabase.SetReplicationActivity(ReplicationActivity))
					{
						MIGRATE_SET_ERROR_RESULT_AND_RETURN("Failed to set replication activity '%s' on database at '%s': %s", *LexToString(InActivityId), *DestDatabase.GetFilename(), *DestDatabase.GetLastError());
					}
				}
			}
			break;

			default:
				checkf(false, TEXT("Unhandled EConcertSyncActivityEventType when migrating session database"));
				break;
			}

			return true;
		});
	}

	DestDatabase.Close();

#undef MIGRATE_SET_ERROR_RESULT_AND_RETURN
	return bResult;
}

} // namespace ConcertSyncServerUtils

FConcertSyncServer::FConcertSyncServer(const FString& InRole, const FConcertSessionFilter& InAutoArchiveSessionFilter)
	: ConcertServer(IConcertServerModule::Get().CreateServer(InRole, InAutoArchiveSessionFilter, this))
	, SessionFlags(EConcertSyncSessionFlags::None)
	, LogReplicationStreamsConsoleCommand(
		TEXT("Replication.LogStreams"),
		TEXT("Logs the streams registered by all clients"),
		FConsoleCommandDelegate::CreateRaw(this, &FConcertSyncServer::LogReplicationStreams))
	, LogReplicationAuthorityConsoleCommand(
		TEXT("Replication.LogAutority"),
		TEXT("Logs the authority of the clients"),
		FConsoleCommandDelegate::CreateRaw(this, &FConcertSyncServer::LogReplicationAuthority))
{}

FConcertSyncServer::~FConcertSyncServer()
{
}

void FConcertSyncServer::Startup(const UConcertServerConfig* InServerConfig, const EConcertSyncSessionFlags InSessionFlags)
{
	SCOPED_CONCERT_TRACE(ConcertServer_Startup);
	SessionFlags = InSessionFlags;

	// Boot the server instance
	ConcertServer->Configure(InServerConfig);
	ConcertServer->Startup();
}

void FConcertSyncServer::Shutdown()
{
	ConcertServer->Shutdown();
}

IConcertServerRef FConcertSyncServer::GetConcertServer() const
{
	return ConcertServer;
}

void FConcertSyncServer::SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService)
{
	ConcertServer->SetFileSharingService(MoveTemp(InFileSharingService));
}

TOptional<FConcertSyncSessionDatabaseNonNullPtr> FConcertSyncServer::GetLiveSessionDatabase(const FGuid& SessionId)
{
	if (const TSharedPtr<FConcertSyncServerLiveSession>* Session = LiveSessions.Find(SessionId))
	{
		return { &Session->Get()->GetSessionDatabase() };
	}
	return {};
}

TOptional<FConcertSyncSessionDatabaseNonNullPtr> FConcertSyncServer::GetArchivedSessionDatabase(const FGuid& SessionId)
{
	if (const TSharedPtr<FConcertSyncServerArchivedSession>* Session = ArchivedSessions.Find(SessionId))
	{
		return { &Session->Get()->GetSessionDatabase() };
	}
	return {};
}

void FConcertSyncServer::GetSessionsFromPathImpl(const IConcertServer& InServer, const FString& InPath, TArray<FConcertSessionInfo>& OutSessionInfos, TArray<FDateTime>* OutSessionCreationTimes)
{
	IFileManager::Get().IterateDirectory(*InPath, [&OutSessionInfos, &OutSessionCreationTimes](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
	{
		if (bIsDirectory)
		{
			const FString SessionRoot = FilenameOrDirectory;

			FConcertSessionInfo SessionInfo;
			if (ConcertSyncServerUtils::ReadSessionInfoFromDirectory(SessionRoot, SessionInfo))
			{
				const FString SessionFolderName = FPaths::GetBaseFilename(SessionRoot);

				// The folder name must be a valid GUID and match the session ID
				FGuid SessionFolderGuid;
				if (FGuid::Parse(SessionFolderName, SessionFolderGuid) && SessionFolderGuid == SessionInfo.SessionId)
				{
					OutSessionInfos.Add(MoveTemp(SessionInfo));
					if (OutSessionCreationTimes)
					{
						OutSessionCreationTimes->Add(ConcertSyncServerUtils::GetSessionCreationTime(SessionRoot));
					}
				}
			}
		}
		return true;
	});
}

bool FConcertSyncServer::OnLiveSessionCreatedImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InSession, const FInternalLiveSessionCreationParams& AdditionalParams)
{
	ConcertSyncServerUtils::WriteSessionInfoToDirectory(InSession->GetSessionWorkingDirectory(), InSession->GetSessionInfo());
	return CreateLiveSession(InSession, AdditionalParams);
}

void FConcertSyncServer::OnLiveSessionDestroyedImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InSession)
{
	DestroyLiveSession(InSession);
}

bool FConcertSyncServer::OnArchivedSessionCreatedImpl(const IConcertServer& InServer, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo)
{
	ConcertSyncServerUtils::WriteSessionInfoToDirectory(InArchivedSessionRoot, InArchivedSessionInfo);
	return CreateArchivedSession(InArchivedSessionRoot, InArchivedSessionInfo);
}

void FConcertSyncServer::OnArchivedSessionDestroyedImpl(const IConcertServer& InServer, const FGuid& InArchivedSessionId)
{
	DestroyArchivedSession(InArchivedSessionId);
}

bool FConcertSyncServer::ArchiveSessionImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo, const FConcertSessionFilter& InSessionFilter)
{
	SCOPED_CONCERT_TRACE(ConcertServer_ArchiveSession);
	return CopySessionImpl(InServer, InLiveSession, InArchivedSessionRoot, InSessionFilter);
}

bool FConcertSyncServer::ArchiveSessionImpl(const IConcertServer& InServer, const FString& InLiveSessionWorkingDir, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo, const FConcertSessionFilter& InSessionFilter)
{
	FConcertSyncSessionDatabase LiveSessionDatabase;
	LiveSessionDatabase.Open(InLiveSessionWorkingDir);

	ConcertSyncServerUtils::WriteSessionInfoToDirectory(InArchivedSessionRoot, InArchivedSessionInfo);
	bool RetVal = ConcertSyncServerUtils::MigrateSessionData(LiveSessionDatabase, InArchivedSessionRoot, InSessionFilter);

	LiveSessionDatabase.Close();
	return RetVal;
}

bool FConcertSyncServer::CopySessionImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FString& NewSessionRoot, const FConcertSessionFilter& InSessionFilter)
{
	SCOPED_CONCERT_TRACE(ConcertServer_CopySession);
	if (TSharedPtr<FConcertSyncServerLiveSession> LiveSession = LiveSessions.FindRef(InLiveSession->GetId()))
	{
		return ConcertSyncServerUtils::MigrateSessionData(LiveSession->GetSessionDatabase(), NewSessionRoot, InSessionFilter);
	}
	return false;
}

bool FConcertSyncServer::ExportSessionImpl(const IConcertServer& InServer, const FGuid& InSessionId, const FString& DestDir, const FConcertSessionFilter& InSessionFilter, bool bAnonymizeData)
{
	SCOPED_CONCERT_TRACE(ConcertServer_ExportSession);
	if (TSharedPtr<FConcertSyncServerLiveSession> LiveSession = LiveSessions.FindRef(InSessionId)) // If the session is live.
	{
		ConcertSyncServerUtils::WriteSessionInfoToDirectory(DestDir, LiveSession->GetSession().GetSessionInfo());
		return ConcertSyncServerUtils::MigrateSessionData(LiveSession->GetSessionDatabase(), DestDir, InSessionFilter);
	}
	else if (TSharedPtr<FConcertSyncServerArchivedSession> ArchivedSession = ArchivedSessions.FindRef(InSessionId)) // If the session is archived.
	{
		ConcertSyncServerUtils::WriteSessionInfoToDirectory(DestDir, ArchivedSession->GetSessionInfo());
		return ConcertSyncServerUtils::MigrateSessionData(ArchivedSession->GetSessionDatabase(), DestDir, InSessionFilter);
	}

	return false; // Session not found.
}

bool FConcertSyncServer::RestoreSessionImpl(const IConcertServer& InServer, const FGuid& InArchivedSessionId, const FString& InLiveSessionRoot, const FConcertSessionInfo& InLiveSessionInfo, const FConcertSessionFilter& InSessionFilter)
{
	SCOPED_CONCERT_TRACE(ConcertServer_RestoreSession);

	if (TSharedPtr<FConcertSyncServerArchivedSession> ArchivedSession = ArchivedSessions.FindRef(InArchivedSessionId))
	{
		return ConcertSyncServerUtils::MigrateSessionData(ArchivedSession->GetSessionDatabase(), InLiveSessionRoot, InSessionFilter);
	}
	return false;
}

bool FConcertSyncServer::GetSessionActivitiesImpl(const IConcertServer& InServer, const FGuid& SessionId, int64 FromActivityId, int64 ActivityCount, TArray<FConcertSessionSerializedPayload>& Activities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, bool bIncludeDetails)
{
	SCOPED_CONCERT_TRACE(ConcertServer_GetSessionActivities);
	if (TSharedPtr<FConcertSyncServerLiveSession> LiveSession = LiveSessions.FindRef(SessionId))
	{
		return GetSessionActivitiesInternal(LiveSession->GetSessionDatabase(), FromActivityId, ActivityCount, Activities, OutEndpointClientInfoMap, bIncludeDetails);
	}
	else if (TSharedPtr<FConcertSyncServerArchivedSession> ArchivedSession = ArchivedSessions.FindRef(SessionId))
	{
		return GetSessionActivitiesInternal(ArchivedSession->GetSessionDatabase(), FromActivityId, ActivityCount, Activities, OutEndpointClientInfoMap, bIncludeDetails);
	}

	return false; // Not Found.
}

bool FConcertSyncServer::GetSessionActivitiesInternal(const FConcertSyncSessionDatabase& Database, int64 FromActivityId, int64 ActivityCount, TArray<FConcertSessionSerializedPayload>& OutActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, bool bIncludeDetails)
{
	int64 MaxActivityId;
	Database.GetActivityMaxId(MaxActivityId);

	if (ActivityCount < 0) // Client requested the 'ActivityCount' last activities?
	{
		ActivityCount = FMath::Abs(ActivityCount);
		FromActivityId = FMath::Max(1ll, MaxActivityId - ActivityCount + 1); // Note: Activity IDs are 1-based, not 0-based.
	}

	OutEndpointClientInfoMap.Reset();
	OutActivities.Reset(FMath::Min(ActivityCount, MaxActivityId));

	// Retrieve the generic part of activities.
	Database.EnumerateActivityIdsWithEventTypesAndFlagsInRange(FromActivityId, ActivityCount, [&Database, &OutActivities, &OutEndpointClientInfoMap, bIncludeDetails](const int64 InActivityId, const EConcertSyncActivityEventType InEventType, const EConcertSyncActivityFlags InFlags)
	{
		// Maps endpoint client id to the client info.
		auto UpdateEndpointMap = [](const FConcertSyncSessionDatabase& Database, FGuid EndpointId, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap)
		{
			FConcertSyncEndpointData EndpointData;
			if (Database.GetEndpoint(EndpointId, EndpointData))
			{
				OutEndpointClientInfoMap.Add(EndpointId, EndpointData.ClientInfo);
			}
		};

		FConcertSessionSerializedPayload SerializedSyncActivityPayload;

		if (InEventType == EConcertSyncActivityEventType::Transaction)
		{
			FConcertSyncTransactionActivity SyncActivity;
			if (Database.GetActivity(InActivityId, SyncActivity)) // Get the generic part of the activity.
			{
				UpdateEndpointMap(Database, SyncActivity.EndpointId, OutEndpointClientInfoMap);

				// If the details are requested, get the transaction data, to inspect what properties were affected by this transaction.
				if (bIncludeDetails)
				{
					Database.GetTransactionEvent(SyncActivity.EventId, SyncActivity.EventData, /*bMetaDataOnly*/false); // Transaction data is required to inspect what property changed.
				}

				SerializedSyncActivityPayload.SetTypedPayload(SyncActivity);
			}
		}
		else if (InEventType == EConcertSyncActivityEventType::Package)
		{
			FConcertSyncPackageActivity SyncActivity;
			if (Database.GetActivity(InActivityId, SyncActivity)) // Get the generic part of the activity.
			{
				UpdateEndpointMap(Database, SyncActivity.EndpointId, OutEndpointClientInfoMap);

				// If the details are requested, get the package event meta-data, which contain extra info about the package name/version, etc.
				if (bIncludeDetails)
				{
					// Don't pull the package data, it can be huge (few GB) and is not useful in the context of this function design. (It is used to display extra info in UI and parsing the large package data is not practical)
					Database.GetPackageEventMetaData(SyncActivity.EventId, SyncActivity.EventData.PackageRevision, SyncActivity.EventData.Package.Info);
				}

				SerializedSyncActivityPayload.SetTypedPayload(SyncActivity);
			}
		}
		else // Connection/lock -> Don't have interesting info to display outside the generic activity info.
		{
			FConcertSyncActivity SyncActivity;
			if (Database.GetActivity(InActivityId, SyncActivity)) // Get the generic part of the activity.
			{
				UpdateEndpointMap(Database, SyncActivity.EndpointId, OutEndpointClientInfoMap);
				SerializedSyncActivityPayload.SetTypedPayload(SyncActivity);
			}
		}

		const bool bIsAllowed = (InFlags & EConcertSyncActivityFlags::Muted) == EConcertSyncActivityFlags::None; 
		if (bIsAllowed)
		{
			OutActivities.Add(MoveTemp(SerializedSyncActivityPayload));
		}

		return true; // Continue until 'ActivityCount' is fetched or the last activity is reached.
	});

	return true;
}

void FConcertSyncServer::OnLiveSessionRenamedImpl(const IConcertServer& InServer,TSharedRef<IConcertServerSession> InLiveSession)
{
	ConcertSyncServerUtils::WriteSessionInfoToDirectory(InLiveSession->GetSessionWorkingDirectory(), InLiveSession->GetSessionInfo());
}

void FConcertSyncServer::OnArchivedSessionRenamedImpl(const IConcertServer& InServer, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo)
{
	ConcertSyncServerUtils::WriteSessionInfoToDirectory(InArchivedSessionRoot, InArchivedSessionInfo);
}

TSharedRef<FConcertServerWorkspace> FConcertSyncServer::CreateWorkspace(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession)
{
	check(InLiveSession->IsValidSession());
	DestroyWorkspace(InLiveSession);
	
	const TSharedRef<FConcertServerWorkspace> Workspace = MakeShared<FConcertServerWorkspace>(InLiveSession, ConcertServer->GetFileSharingService());
	LiveSessionWorkspaces.Add(InLiveSession->GetSession().GetId(), Workspace);
	return Workspace;
}

void FConcertSyncServer::DestroyWorkspace(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession)
{
	LiveSessionWorkspaces.Remove(InLiveSession->GetSession().GetId());
}

void FConcertSyncServer::CreateSequencerManager(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession)
{
	check(InLiveSession->IsValidSession());
	DestroySequencerManager(InLiveSession);
	LiveSessionSequencerManagers.Add(InLiveSession->GetSession().GetId(), MakeShared<FConcertServerSequencerManager>(InLiveSession));
}

void FConcertSyncServer::DestroySequencerManager(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession)
{
	LiveSessionSequencerManagers.Remove(InLiveSession->GetSession().GetId());
}

void FConcertSyncServer::CreateReplicationManager(const TSharedRef<IConcertServerSession>& InSession, UE::ConcertSyncServer::Replication::IReplicationWorkspace& InWorkspace, EConcertSyncSessionFlags InSessionFlags)
{
	const FGuid& SessionId = InSession->GetId(); 
	DestroyReplicationManager(SessionId);
	LiveSessionReplicationManagers.Add(
		SessionId,
		MakeShared<UE::ConcertSyncServer::Replication::FConcertServerReplicationManager>(InSession, InWorkspace, InSessionFlags)
		);
}

void FConcertSyncServer::DestroyReplicationManager(const FGuid& SessionId)
{
	LiveSessionReplicationManagers.Remove(SessionId);
}

bool FConcertSyncServer::CreateLiveSession(const TSharedRef<IConcertServerSession>& InSession, const FInternalLiveSessionCreationParams& AdditionalParams)
{
	DestroyLiveSession(InSession);

	TSharedPtr<FConcertSyncServerLiveSession> LiveSession = MakeShared<FConcertSyncServerLiveSession>(InSession, SessionFlags);
	if (LiveSession->IsValidSession())
	{
		LiveSessions.Add(InSession->GetId(), LiveSession);
		const TSharedRef<FConcertServerWorkspace> Workspace = CreateWorkspace(LiveSession.ToSharedRef());
		if (EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::EnableSequencer))
		{
			CreateSequencerManager(LiveSession.ToSharedRef());
		}

		// Create Replication Manager
		if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableReplication))
		{
			CreateReplicationManager(InSession, Workspace->GetReplicationWorkspace(), LiveSession->GetSessionFlags());
		}

		// We needn't call OnActivityProduced().Remove(...) because the subscription needs to stay for the lifetime of FConcertSyncServerLiveSession::SessionDatabase
		check(AdditionalParams.OnModifiedCallback.IsBound());
		LiveSession->GetSessionDatabase().OnActivityProduced().AddLambda([OnSessionModifiedCallback = AdditionalParams.OnModifiedCallback](const FConcertSyncActivity&)
		{
			OnSessionModifiedCallback.Execute();
		});
		return true;
	}

	return false;
}

void FConcertSyncServer::DestroyLiveSession(const TSharedRef<IConcertServerSession>& InSession)
{
	if (TSharedPtr<FConcertSyncServerLiveSession> LiveSession = LiveSessions.FindRef(InSession->GetId()))
	{
		DestroyWorkspace(LiveSession.ToSharedRef());
		DestroySequencerManager(LiveSession.ToSharedRef());
		DestroyReplicationManager(InSession->GetId());
		LiveSessions.Remove(InSession->GetId());
	}
}

bool FConcertSyncServer::CreateArchivedSession(const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo)
{
	DestroyArchivedSession(InArchivedSessionInfo.SessionId);

	TSharedPtr<FConcertSyncServerArchivedSession> ArchivedSession = MakeShared<FConcertSyncServerArchivedSession>(InArchivedSessionRoot, InArchivedSessionInfo);
	if (ArchivedSession->IsValidSession())
	{
		ArchivedSessions.Add(ArchivedSession->GetId(), ArchivedSession);
		return true;
	}

	return false;
}

void FConcertSyncServer::DestroyArchivedSession(const FGuid& InArchivedSessionId)
{
	ArchivedSessions.Remove(InArchivedSessionId);
}

void FConcertSyncServer::LogReplicationStreams() const
{
	UE_LOGF(LogConcert, Log, "Printing replication streams...");
	
	TWideStringBuilder<2048> StringBuilder;
	constexpr auto SessionIntent = TEXT("\n");
	constexpr auto ClientIndent = TEXT("\n\t");
	constexpr auto StreamIndent = TEXT("\n\t\t");
	constexpr auto ObjectIndent = TEXT("\n\t\t\t");
	constexpr auto PropertyIndent = TEXT("\n\t\t\t\t");
	
	for (const TPair<FGuid, TSharedRef<UE::ConcertSyncServer::Replication::FConcertServerReplicationManager>>& ReplicationManagerPair : LiveSessionReplicationManagers)
	{
		const TSharedPtr<FConcertSyncServerLiveSession>* LiveSession = LiveSessions.Find(ReplicationManagerPair.Key);
		check(LiveSession);

		StringBuilder << SessionIntent << TEXT("Session: ") << *LiveSession->Get()->GetSession().GetName() << TEXT(" (") << *ReplicationManagerPair.Key.ToString() << TEXT(")");
		for (const FConcertSessionClientInfo& ClientInfo : LiveSession->Get()->GetSession().GetSessionClients())
		{
			const FGuid ClientId = ClientInfo.ClientEndpointId;
			StringBuilder << ClientIndent << TEXT("Client: ") << *ClientInfo.ClientInfo.DisplayName << TEXT(" (") << ClientId.ToString() << TEXT(")");
			ReplicationManagerPair.Value->ForEachStream(ClientId, [&StringBuilder, StreamIndent, ObjectIndent, PropertyIndent](const FConcertReplicationStream& Stream)
			{
				StringBuilder << StreamIndent << TEXT("Stream: ") << *Stream.BaseDescription.Identifier.ToString();
				for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ReplicatedObject : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
				{
					StringBuilder << ObjectIndent << *ReplicatedObject.Key.ToString();
					for (const FConcertPropertyChain& Property : ReplicatedObject.Value.PropertySelection.ReplicatedProperties)
					{
						StringBuilder << PropertyIndent << Property.ToString();
					}
				}
				return EBreakBehavior::Continue;
			});
		}
	}
	
	UE_LOGF(LogConcert, Log, "%ls", StringBuilder.ToString());
	// The logs before and after are to have some feedback in case nothing is printed (e.g. no live sessions).
	UE_LOGF(LogConcert, Log, "Done.");
}

void FConcertSyncServer::LogReplicationAuthority() const
{
	using namespace UE::ConcertSyncServer::Replication;
	UE_LOGF(LogConcert, Log, "Printing replication authority...");
	
	TWideStringBuilder<2048> StringBuilder;
	constexpr auto SessionIntent = TEXT("\n");
	constexpr auto ClientIndent = TEXT("\n\t");
	constexpr auto StreamIndent = TEXT("\n\t\t");
	constexpr auto ObjectIndent = TEXT("\n\t\t\t");
	
	for (const TPair<FGuid, TSharedRef<FConcertServerReplicationManager>>& ReplicationManagerPair : LiveSessionReplicationManagers)
	{
		const TSharedPtr<FConcertSyncServerLiveSession>* LiveSession = LiveSessions.Find(ReplicationManagerPair.Key);
		check(LiveSession);

		StringBuilder << SessionIntent << TEXT("Session: ") << *LiveSession->Get()->GetSession().GetName() << TEXT(" (") << *ReplicationManagerPair.Key.ToString() << TEXT(")");
		for (const FConcertSessionClientInfo& ClientInfo : LiveSession->Get()->GetSession().GetSessionClients())
		{
			const FGuid ClientId = ClientInfo.ClientEndpointId;
			StringBuilder << ClientIndent << TEXT("Client: ") << *ClientInfo.ClientInfo.DisplayName << TEXT(" (") << ClientId.ToString() << TEXT(")");
			
			const TSharedRef<FConcertServerReplicationManager>& ReplicationManager = ReplicationManagerPair.Value;
			ReplicationManager->ForEachStream(ClientId, [&StringBuilder, StreamIndent, ObjectIndent, ClientId, &ReplicationManager](const FConcertReplicationStream& Stream)
			{
				const FGuid StreamId = Stream.BaseDescription.Identifier;
				StringBuilder << StreamIndent << TEXT("Stream: ") << *StreamId.ToString();
				ReplicationManager->GetAuthorityManager().EnumerateAuthority(ClientId, StreamId, [&StringBuilder, ObjectIndent](const FSoftObjectPath& AuthoredObject)
				{
					StringBuilder << ObjectIndent << *AuthoredObject.ToString();
					return EBreakBehavior::Continue;
				});
				return EBreakBehavior::Continue;
			});
		}
	}
	
	UE_LOGF(LogConcert, Log, "%ls", StringBuilder.ToString());
	// The logs before and after are to have some feedback in case nothing is printed (e.g. no live sessions).
	UE_LOGF(LogConcert, Log, "Done.");
}

#undef LOCTEXT_NAMESPACE
