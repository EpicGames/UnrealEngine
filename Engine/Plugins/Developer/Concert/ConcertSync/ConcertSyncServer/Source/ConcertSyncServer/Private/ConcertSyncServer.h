// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertServerEventForwardingSink.h"
#include "IConcertSyncServer.h"
#include "IConcertServerEventSink.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"

class IConcertServerSession;
class FConcertServerWorkspace;
class FConcertServerSequencerManager;
class FConcertSyncServerLiveSession;
class FConcertSyncServerArchivedSession;
class FConcertSyncSessionDatabase;

namespace UE::ConcertSyncServer::Replication
{
	class FConcertServerReplicationManager;
	class IReplicationWorkspace;
}

struct FConcertSessionFilter;

/**
 * Implementation for a Concert Sync Server.
 */
class FConcertSyncServer : public IConcertSyncServer, public TConcertServerEventForwardingSink<FConcertSyncServer>
{
public:
	
	FConcertSyncServer(const FString& InRole, const FConcertSessionFilter& InAutoArchiveSessionFilter);
	virtual ~FConcertSyncServer();

	//~ IConcertSyncServer interface
	virtual void Startup(const UConcertServerConfig* InServerConfig, const EConcertSyncSessionFlags InSessionFlags) override;
	virtual void Shutdown() override;
	virtual IConcertServerRef GetConcertServer() const override;
	virtual void SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService) override;
	virtual TOptional<FConcertSyncSessionDatabaseNonNullPtr> GetLiveSessionDatabase(const FGuid& SessionId) override;
	virtual TOptional<FConcertSyncSessionDatabaseNonNullPtr> GetArchivedSessionDatabase(const FGuid& SessionId) override;

	//~ IConcertServerEventSink interface
	void GetSessionsFromPathImpl(const IConcertServer& InServer, const FString& InPath, TArray<FConcertSessionInfo>& OutSessionInfos, TArray<FDateTime>* OutSessionCreationTimes = nullptr);
	bool OnLiveSessionCreatedImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FInternalLiveSessionCreationParams& AdditionalParams);
	void OnLiveSessionDestroyedImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession);
	bool OnArchivedSessionCreatedImpl(const IConcertServer& InServer, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo);
	void OnArchivedSessionDestroyedImpl(const IConcertServer& InServer, const FGuid& InArchivedSessionId);
	bool ArchiveSessionImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo, const FConcertSessionFilter& InSessionFilter);
	bool ArchiveSessionImpl(const IConcertServer& InServer, const FString& InLiveSessionWorkingDir, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo, const FConcertSessionFilter& InSessionFilter);
	bool CopySessionImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession, const FString& NewSessionRoot, const FConcertSessionFilter& InSessionFilter);
	bool ExportSessionImpl(const IConcertServer& InServer, const FGuid& InSessionId, const FString& DestDir, const FConcertSessionFilter& InSessionFilter, bool bAnonymizeData);
	bool RestoreSessionImpl(const IConcertServer& InServer, const FGuid& InArchivedSessionId, const FString& InLiveSessionRoot, const FConcertSessionInfo& InLiveSessionInfo, const FConcertSessionFilter& InSessionFilter);
	bool GetSessionActivitiesImpl(const IConcertServer& InServer, const FGuid& SessionId, int64 FromActivityId, int64 ActivityCount, TArray<FConcertSessionSerializedPayload>& OutActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, bool bIncludeDetails);
	void OnLiveSessionRenamedImpl(const IConcertServer& InServer, TSharedRef<IConcertServerSession> InLiveSession);
	void OnArchivedSessionRenamedImpl(const IConcertServer& InServer, const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo);

private:
	
	TSharedRef<FConcertServerWorkspace> CreateWorkspace(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession);
	void DestroyWorkspace(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession);

	void CreateSequencerManager(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession);
	void DestroySequencerManager(const TSharedRef<FConcertSyncServerLiveSession>& InLiveSession);

	/**
	 * Initializes the replication manager for the given session.
	 * @param InSession The session to bind the manager to
	 * @param InWorkspace The database interface that manager interacts with. The caller ensures it outlives the lifetime of the widget.
	 * @param InSessionFlags Session flags, which determine optional features.
	 */
	void CreateReplicationManager(
		const TSharedRef<IConcertServerSession>& InSession, UE::ConcertSyncServer::Replication::IReplicationWorkspace& InWorkspace,
		EConcertSyncSessionFlags InSessionFlags
		);
	void DestroyReplicationManager(const FGuid& SessionId); 

	bool CreateLiveSession(const TSharedRef<IConcertServerSession>& InSession, const FInternalLiveSessionCreationParams& AdditionalParams);
	void DestroyLiveSession(const TSharedRef<IConcertServerSession>& InSession);

	bool CreateArchivedSession(const FString& InArchivedSessionRoot, const FConcertSessionInfo& InArchivedSessionInfo);
	void DestroyArchivedSession(const FGuid& InArchivedSessionId);

	bool GetSessionActivitiesInternal(const FConcertSyncSessionDatabase& Database, int64 FromActivityId, int64 ActivityCount, TArray<FConcertSessionSerializedPayload>& OutActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, bool bIncludeDetails);

	/** Server for Concert */
	IConcertServerRef ConcertServer;

	/** Flags controlling what features are enabled for sessions within this server */
	EConcertSyncSessionFlags SessionFlags;

	/** Map of live session IDs to their associated workspaces */
	TMap<FGuid, TSharedPtr<FConcertServerWorkspace>> LiveSessionWorkspaces;

	/** Map of live session IDs to their associated sequencer managers */
	TMap<FGuid, TSharedPtr<FConcertServerSequencerManager>> LiveSessionSequencerManagers;

	/** Map of live session IDs to their associated replication managers */
	TMap<FGuid, TSharedRef<UE::ConcertSyncServer::Replication::FConcertServerReplicationManager>> LiveSessionReplicationManagers;

	/** Map of live session IDs to their associated session data */
	TMap<FGuid, TSharedPtr<FConcertSyncServerLiveSession>> LiveSessions;

	/** Map of archived session IDs to their associated session data */
	TMap<FGuid, TSharedPtr<FConcertSyncServerArchivedSession>> ArchivedSessions;

	/** Logs the replication streams registered by all clients */
	FAutoConsoleCommand LogReplicationStreamsConsoleCommand;
	/** Logs the replication authority of the clients */
	FAutoConsoleCommand LogReplicationAuthorityConsoleCommand;

	/** Logs the replication streams registered by all clients */
	void LogReplicationStreams() const;
	/** Logs the replication authority of the clients */
	void LogReplicationAuthority() const;
};
