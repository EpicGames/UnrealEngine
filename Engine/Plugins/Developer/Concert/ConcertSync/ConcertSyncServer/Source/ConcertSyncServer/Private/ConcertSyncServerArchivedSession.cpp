// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncServerArchivedSession.h"
#include "ConcertSyncSessionDatabase.h"

FConcertSyncServerArchivedSession::FConcertSyncServerArchivedSession(const FString& InSessionRootPath, const FConcertSessionInfo& InSessionInfo)
	: SessionRootPath(InSessionRootPath)
	, SessionInfo(InSessionInfo)
	, SessionDatabase(MakeUnique<FConcertSyncSessionDatabase>())
{
	SessionDatabase->Open(SessionRootPath);
}

FConcertSyncServerArchivedSession::~FConcertSyncServerArchivedSession()
{
	SessionDatabase->Close();
	SessionDatabase.Reset();
}

bool FConcertSyncServerArchivedSession::IsValidSession() const
{
	return SessionDatabase->IsValid();
}
