// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"

class FConcertSyncSessionDatabase;

class FConcertSyncServerArchivedSession
{
public:
	FConcertSyncServerArchivedSession(const FString& InSessionRootPath, const FConcertSessionInfo& InSessionInfo);
	~FConcertSyncServerArchivedSession();

	FConcertSyncServerArchivedSession(const FConcertSyncServerArchivedSession&) = delete;
	FConcertSyncServerArchivedSession& operator=(const FConcertSyncServerArchivedSession&) = delete;

	FConcertSyncServerArchivedSession(FConcertSyncServerArchivedSession&&) = delete;
	FConcertSyncServerArchivedSession& operator=(FConcertSyncServerArchivedSession&&) = delete;

	/** Is this a valid session? (ie, has been successfully opened) */
	bool IsValidSession() const;

	/** Get the ID of this archived session */
	const FGuid& GetId() const
	{
		return SessionInfo.SessionId;
	}

	/** Get the name of this archived session */
	const FString& GetName() const
	{
		return SessionInfo.SessionName;
	}

	/** Get the information about this archived session */
	const FConcertSessionInfo& GetSessionInfo() const
	{
		return SessionInfo;
	}

	/** Get the root path of this archived session */
	FString GetSessionWorkingDirectory() const
	{
		return SessionRootPath;
	}

	/** Get the database for this archived session (if IsValidSession) */
	FConcertSyncSessionDatabase& GetSessionDatabase()
	{
		return *SessionDatabase;
	}

	/** Get the database for this archived session (if IsValidSession) */
	const FConcertSyncSessionDatabase& GetSessionDatabase() const
	{
		return *SessionDatabase;
	}

private:
	/** Root path of this archived session */
	FString SessionRootPath;

	/** Information about this archived session */
	FConcertSessionInfo SessionInfo;

	/** Database for this archived session */
	TUniquePtr<FConcertSyncSessionDatabase> SessionDatabase;
};
