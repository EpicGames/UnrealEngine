// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncServerLiveSession.h"
#include "ConcertSyncSessionDatabase.h"
#include "IConcertSession.h"

FConcertSyncServerLiveSession::FConcertSyncServerLiveSession(const TSharedRef<IConcertServerSession>& InSession, const EConcertSyncSessionFlags InSessionFlags)
	: Session(InSession)
	, SessionFlags(InSessionFlags)
	, SessionDatabase(MakeUnique<FConcertSyncSessionDatabase>())
{
	SessionDatabase->Open(Session->GetSessionWorkingDirectory());
}

FConcertSyncServerLiveSession::~FConcertSyncServerLiveSession()
{
	SessionDatabase->Close();
	SessionDatabase.Reset();
}

bool FConcertSyncServerLiveSession::IsValidSession() const
{
	return SessionDatabase->IsValid();
}
