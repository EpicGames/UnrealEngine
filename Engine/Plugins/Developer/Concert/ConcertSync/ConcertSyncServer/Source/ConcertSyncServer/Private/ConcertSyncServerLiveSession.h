// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"
#include "ConcertSyncSessionFlags.h"

class IConcertServerSession;
class FConcertSyncSessionDatabase;

class FConcertSyncServerLiveSession
{
public:
	FConcertSyncServerLiveSession(const TSharedRef<IConcertServerSession>& InSession, const EConcertSyncSessionFlags InSessionFlags);
	~FConcertSyncServerLiveSession();

	FConcertSyncServerLiveSession(const FConcertSyncServerLiveSession&) = delete;
	FConcertSyncServerLiveSession& operator=(const FConcertSyncServerLiveSession&) = delete;

	FConcertSyncServerLiveSession(FConcertSyncServerLiveSession&&) = delete;
	FConcertSyncServerLiveSession& operator=(FConcertSyncServerLiveSession&&) = delete;

	/** Is this a valid session? (ie, has been successfully opened) */
	bool IsValidSession() const;

	/** Get the underlying live session (if IsValidSession) */
	IConcertServerSession& GetSession()
	{
		return *Session;
	}

	/** Get the underlying live session (if IsValidSession) */
	const IConcertServerSession& GetSession() const
	{
		return *Session;
	}

	/** Get the flags controlling what features are enabled for this live session */
	EConcertSyncSessionFlags GetSessionFlags() const
	{
		return SessionFlags;
	}

	/** Get the database for this live session (if IsValidSession) */
	FConcertSyncSessionDatabase& GetSessionDatabase()
	{
		return *SessionDatabase;
	}

	/** Get the database for this live session (if IsValidSession) */
	const FConcertSyncSessionDatabase& GetSessionDatabase() const
	{
		return *SessionDatabase;
	}

private:
	/** The underlying live session */
	TSharedRef<IConcertServerSession> Session;

	/** Flags controlling what features are enabled for this live session */
	EConcertSyncSessionFlags SessionFlags;

	/** Database for this live session */
	TUniquePtr<FConcertSyncSessionDatabase> SessionDatabase;
};
