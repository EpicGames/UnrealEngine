// Copyright Epic Games, Inc. All Rights Reserved.

#include "IConcertSyncServerModule.h"
#include "ConcertSyncServer.h"
#include "ConcertSettings.h"
#include "ConcertServerSettings.h"
#include "Logging/LogVerbosity.h"
#include "Misc/EngineVersion.h"
#include "ConcertLogGlobal.h"

#include "HAL/FileManager.h"

#include "Misc/Paths.h"

#include "ConcertCloudFileSharingService.h"

/**
 * 
 */
class FConcertSyncServerModule : public IConcertSyncServerModule
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

	virtual UConcertServerConfig* ParseServerSettings(const TCHAR* CommandLine) override
	{
		UConcertServerConfig* ServerConfig = NewObject<UConcertServerConfig>();

		if (CommandLine)
		{
			// Parse value overrides (if present)
			FParse::Value(CommandLine, TEXT("-CONCERTSERVER="), ServerConfig->ServerName);
			FParse::Value(CommandLine, TEXT("-CONCERTSESSION="), ServerConfig->DefaultSessionName);
			FParse::Value(CommandLine, TEXT("-CONCERTSESSIONTORESTORE="), ServerConfig->DefaultSessionToRestore);
			FParse::Value(CommandLine, TEXT("-CONCERTSAVESESSIONAS="), ServerConfig->DefaultSessionSettings.ArchiveNameOverride);
			FParse::Value(CommandLine, TEXT("-CONCERTPROJECT="), ServerConfig->DefaultSessionSettings.ProjectName);
			FParse::Value(CommandLine, TEXT("-CONCERTREVISION="), ServerConfig->DefaultSessionSettings.BaseRevision);
			FParse::Value(CommandLine, TEXT("-CONCERTWORKINGDIR="), ServerConfig->WorkingDir);
			FParse::Value(CommandLine, TEXT("-CONCERTSAVEDDIR="), ServerConfig->ArchiveDir);
			FParse::Value(CommandLine, TEXT("-CONCERTENDPOINTTIMEOUT="), ServerConfig->EndpointSettings.RemoteEndpointTimeoutSeconds);

			FString VersionString;
			if (FParse::Value(CommandLine, TEXT("-CONCERTVERSION="), VersionString))
			{
				FEngineVersion EngineVersion;
				UE_LOGF(LogConcert, Warning, "CONCERTVERSION command line flag is deprecated and will be removed in a future version.");
				if (FEngineVersion::Parse(VersionString, EngineVersion))
				{
					ServerConfig->DefaultVersionInfo.EngineVersion.Initialize(EngineVersion);
					UE_LOGF(LogConcert, Display, "Override for engine version set to '%ls'.", *VersionString);
				}
				else
				{
					UE_LOGF(LogConcert, Warning, "Failed to parse version string '%ls'.",*VersionString);
				}
			}

			ServerConfig->ServerSettings.bIgnoreSessionSettingsRestriction |= FParse::Param(CommandLine, TEXT("CONCERTIGNORE"));
			FParse::Bool(CommandLine, TEXT("-CONCERTIGNORE="), ServerConfig->ServerSettings.bIgnoreSessionSettingsRestriction);

			ServerConfig->bCleanWorkingDir |= FParse::Param(CommandLine, TEXT("CONCERTCLEAN"));
			FParse::Bool(CommandLine, TEXT("-CONCERTCLEAN="), ServerConfig->bCleanWorkingDir);

			ServerConfig->EndpointSettings.bEnableLogging |= FParse::Param(CommandLine, TEXT("CONCERTLOGGING"));
			FParse::Bool(CommandLine, TEXT("-CONCERTLOGGING="), ServerConfig->EndpointSettings.bEnableLogging);

			ServerConfig->bEnableFileSharing |= FParse::Param(CommandLine, TEXT("CONCERTFILESHARINGENABLED"));
			FParse::Bool(CommandLine, TEXT("-CONCERTFILESHARINGENABLED="), ServerConfig->bEnableFileSharing);

			FParse::Value(CommandLine, TEXT("-CONCERTFILESHAREPATH="), ServerConfig->RootFileSharingPath.Path);
			if (ServerConfig->bEnableFileSharing)
			{
				bool bShouldEnableFileSharing = false;
				if (!ServerConfig->RootFileSharingPath.Path.IsEmpty())
				{
					FString FullPath = FPaths::ConvertRelativePathToFull(ServerConfig->RootFileSharingPath.Path);
					FFileStatData StatData = IFileManager::Get().GetStatData(*FullPath);
					if (!StatData.bIsReadOnly && StatData.bIsValid && StatData.bIsDirectory)
					{
						ServerConfig->RootFileSharingPath.Path = FullPath;
						bShouldEnableFileSharing = true;
					}
					else
					{
						UE_LOGF(LogConcert, Warning, "The file sharing path specified was not a valid writable directory. File sharing cannot be enabled for %ls.", *FullPath);
					}
				}

				ServerConfig->bEnableFileSharing = bShouldEnableFileSharing;
				if (bShouldEnableFileSharing)
				{
					UE_LOGF(LogConcert, Display, "File sharing folder %ls has been specified and will be used for sessions.", *ServerConfig->RootFileSharingPath.Path);
				}
				else
				{
					UE_LOGF(LogConcert, Warning, "The server was launched with file sharing enabled but we were unable to validate file share path (%ls).", *ServerConfig->RootFileSharingPath.Path);
				}
			}
		}

		return ServerConfig;
	}

	virtual TSharedRef<IConcertSyncServer> CreateServer(const FString& InRole, const FConcertSessionFilter& InAutoArchiveSessionFilter) override
	{
		TSharedRef<IConcertSyncServer> ConcertSyncServer = MakeShared<FConcertSyncServer>(InRole, InAutoArchiveSessionFilter);
		OnConcertSyncServerCreatedDelegate.Broadcast(ConcertSyncServer);
		return ConcertSyncServer;
	}

	virtual TSharedRef<IConcertFileSharingService> CreateFileSharingService(const FString& InRole) override
	{
		TSharedRef<IConcertFileSharingService> FileShare = MakeShared<FConcertCloudFileSharingService>(InRole);
		return FileShare;
	}

	virtual FOnConcertSyncServerCreated& OnServerCreated() override
	{
		return OnConcertSyncServerCreatedDelegate;
	}

protected:

	FOnConcertSyncServerCreated OnConcertSyncServerCreatedDelegate;
};

IMPLEMENT_MODULE(FConcertSyncServerModule, ConcertSyncServer);
