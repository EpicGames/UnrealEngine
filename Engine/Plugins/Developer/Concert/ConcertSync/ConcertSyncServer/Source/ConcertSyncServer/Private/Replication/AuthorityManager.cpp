// Copyright Epic Games, Inc. All Rights Reserved.

#include "AuthorityManager.h"

#include "ConcertLogGlobal.h"
#include "IConcertSession.h"
#include "Replication/AuthorityConflictSharedUtils.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Messages/ChangeAuthority.h"
#include "Replication/Misc/IReplicationGroundTruth.h"
#include "Util/GroundTruthOverride.h"
#include "Util/LogUtils.h"
#include "Util/ReplicationCVars.h"

namespace UE::ConcertSyncServer::Replication
{
	namespace Private
	{
		class FServerGroundTruth : public ConcertSyncCore::Replication::IReplicationGroundTruth
		{
			const FAuthorityManager& Owner;
			const IRegistrationEnumerator& Getters;
		public:
			
			FServerGroundTruth(const FAuthorityManager& Owner, const IRegistrationEnumerator& Getters)
				: Owner(Owner)
				, Getters(Getters)
			{}

			virtual void ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)> Callback) const override
			{
				Getters.ForEachStream(ClientEndpointId, [&Callback](const FConcertReplicationStream& Stream)
				{
					return Callback(Stream.BaseDescription.Identifier, Stream.BaseDescription.ReplicationMap);
				});
			}
			
			virtual void ForEachClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const override
			{
				Getters.ForEachReplicationClient(Callback);
			}
			
			virtual bool HasAuthority(const FGuid& ClientId, const FGuid& StreamId, const FSoftObjectPath& ObjectPath) const override
			{
				return Owner.HasAuthorityToChange({ { StreamId, ObjectPath }, ClientId });
			}
		};
		
		static void ForEachReplicatedObject(
			const TMap<FSoftObjectPath, FConcertStreamArray>& Map,
			TFunctionRef<void(const FGuid& StreamId, const FSoftObjectPath& ObjectPath)> Callback
			)
		{
			for (const TPair<FSoftObjectPath, FConcertStreamArray>& ObjectInfo : Map)
			{
				for (const FGuid& StreamId : ObjectInfo.Value.StreamIds)
				{
					Callback(StreamId, ObjectInfo.Key);
				}
			}
		}

		const FConcertReplicationStream* FindClientStreamById(const IStreamEnumerator& Getters, const FGuid& ClientId, const FGuid& StreamId)
		{
			const FConcertReplicationStream* StreamDescription = nullptr;
			Getters.ForEachStream(ClientId, [&StreamId, &StreamDescription](const FConcertReplicationStream& Stream) mutable
			{
				if (Stream.BaseDescription.Identifier == StreamId)
				{
					StreamDescription = &Stream;
					return EBreakBehavior::Break;
				}
				return EBreakBehavior::Continue;
			});
			return StreamDescription;
		}

		const FConcertObjectReplicationMap* FindClientReplicationMapById(
			const ConcertSyncCore::Replication::IReplicationGroundTruth& Getters,
			const FGuid& ClientId,
			const FGuid& StreamId
			)
		{
			const FConcertObjectReplicationMap* ReplicationMap = nullptr;
			Getters.ForEachStream(ClientId, [&StreamId, &ReplicationMap](const FGuid& InStreamId, const FConcertObjectReplicationMap& InReplicationMap) mutable
			{
				if (InStreamId == StreamId)
				{
					ReplicationMap = &InReplicationMap;
					return EBreakBehavior::Break;
				}
				return EBreakBehavior::Continue;
			});
			return ReplicationMap;
		}
	}
	
	FAuthorityManager::FAuthorityManager(
		IRegistrationEnumerator& InGetters,
		TSharedRef<IConcertSession> InSession
		)
		: Getters(InGetters)
		, Session(MoveTemp(InSession))
	{
		Session->RegisterCustomRequestHandler<FConcertReplication_ChangeAuthority_Request, FConcertReplication_ChangeAuthority_Response>(this, &FAuthorityManager::HandleChangeAuthorityRequest);
	}

	FAuthorityManager::~FAuthorityManager()
	{
		Session->UnregisterCustomRequestHandler<FConcertReplication_ChangeAuthority_Request>();
	}

	bool FAuthorityManager::HasAuthorityToChange(const FConcertReplicatedObjectId& ObjectChange) const
	{
		const FClientAuthorityData* AuthorityData = ClientAuthorityData.Find(ObjectChange.SenderEndpointId);
		const TSet<FSoftObjectPath>* OwnedObjects = AuthorityData ? AuthorityData->OwnedObjects.Find(ObjectChange.StreamId) : nullptr;
		return OwnedObjects && OwnedObjects->Contains(ObjectChange.Object);
	}

	void FAuthorityManager::EnumerateAuthority(const FClientId& ClientId, const FStreamId& StreamId, TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Callback) const
	{
		const FClientAuthorityData* AuthorityData = ClientAuthorityData.Find(ClientId);
		const TSet<FSoftObjectPath>* OwnedObjects = AuthorityData ? AuthorityData->OwnedObjects.Find(StreamId) : nullptr;
		if (!OwnedObjects)
		{
			return;
		}
		
		for (const FSoftObjectPath& AuthoredObject : *OwnedObjects)
		{
			if (Callback(AuthoredObject) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	TArray<FConcertObjectInStreamID> FAuthorityManager::GetOwnedObjects(const FClientId& ClientId) const
	{
		TArray<FConcertObjectInStreamID> Result;
		const FClientAuthorityData* AuthorityData = ClientAuthorityData.Find(ClientId);
		if (!AuthorityData)
		{
			return Result;
		}

		for (const TPair<FStreamId, TSet<FSoftObjectPath>>& Pair : AuthorityData->OwnedObjects)
		{
			for (const FSoftObjectPath& ObjectPath : Pair.Value)
			{
				Result.Add({ Pair.Key, ObjectPath });
			}
		}
		
		return Result;
	}

	FAuthorityManager::EAuthorityResult FAuthorityManager::EnumerateAuthorityConflicts(
		const FConcertReplicatedObjectId& Object, 
		const FConcertPropertySelection* OverrideProperties,
		FProcessAuthorityConflict ProcessConflict
		) const
	{
		const FClientId& ClientId = Object.SenderEndpointId;
		const FConcertPropertySelection* PropertiesToCheck = OverrideProperties;
		if (!PropertiesToCheck)
		{
			const FConcertReplicationStream* Description = Private::FindClientStreamById(Getters, ClientId, Object.StreamId);
			const FConcertReplicatedObjectInfo* PropertyInfo = Description ? Description->BaseDescription.ReplicationMap.ReplicatedObjects.Find(Object.Object) : nullptr;
			PropertiesToCheck = PropertyInfo ? &PropertyInfo->PropertySelection : nullptr;
		}

		if (!PropertiesToCheck)
		{
			return EAuthorityResult::NoRegisteredProperties;
		}

		using namespace ConcertSyncCore::Replication;
		using namespace ConcertSyncCore::Replication::AuthorityConflictUtils;
		Private::FServerGroundTruth GroundTruth(*this, Getters);
		const EAuthorityConflict Conflict = AuthorityConflictUtils::EnumerateAuthorityConflicts(
			Object.SenderEndpointId,
			Object.Object,
			PropertiesToCheck->ReplicatedProperties,
			GroundTruth,
			[&ProcessConflict](const FClientId& ClientId, const FStreamId& StreamId, const FConcertPropertyChain& Property)
			{
				return ProcessConflict(ClientId, StreamId, Property);
			});
		return Conflict == EAuthorityConflict::Allowed ? EAuthorityResult::Allowed : EAuthorityResult::Conflict;
	}

	bool FAuthorityManager::CanTakeAuthority(const FConcertReplicatedObjectId& Object) const
	{
		return EnumerateAuthorityConflicts(Object) == EAuthorityResult::Allowed;
	}

	FAuthorityManager::EAuthorityResult FAuthorityManager::EnumerateAuthorityConflictsWithOverrides(
		const FConcertReplicatedObjectId& Object,
		const TMap<FGuid, FConcertReplicationStreamArray>& StreamOverrides,
		const TMap<FGuid, FConcertObjectInStreamArray>& AuthorityOverrides,
		FProcessAuthorityConflict ProcessConflict
		) const
	{
		const FClientId& ClientId = Object.SenderEndpointId;
		const FGroundTruthOverride GroundTruth(StreamOverrides, AuthorityOverrides, Getters, *this);
		
		const FConcertObjectReplicationMap* ReplicationMap = Private::FindClientReplicationMapById(GroundTruth, ClientId, Object.StreamId);
		const FConcertReplicatedObjectInfo* PropertyInfo = ReplicationMap ? ReplicationMap->ReplicatedObjects.Find(Object.Object) : nullptr;
		const FConcertPropertySelection* PropertiesToCheck = PropertyInfo ? &PropertyInfo->PropertySelection : nullptr;
		if (!PropertiesToCheck)
		{
			return EAuthorityResult::NoRegisteredProperties;
		}

		using namespace ConcertSyncCore::Replication;
		using namespace ConcertSyncCore::Replication::AuthorityConflictUtils;
		const EAuthorityConflict Conflict = AuthorityConflictUtils::EnumerateAuthorityConflicts(
			Object.SenderEndpointId,
			Object.Object,
			PropertiesToCheck->ReplicatedProperties,
			GroundTruth,
			[&ProcessConflict](const FClientId& ClientId, const FStreamId& StreamId, const FConcertPropertyChain& Property)
			{
				return ProcessConflict(ClientId, StreamId, Property);
			});
		return Conflict == EAuthorityConflict::Allowed ? EAuthorityResult::Allowed : EAuthorityResult::Conflict;
	}

	void FAuthorityManager::OnPostClientLeft(const FClientId& ClientEndpointId)
	{
		ClientAuthorityData.Remove(ClientEndpointId);
	}

	void FAuthorityManager::RemoveAuthority(const FConcertReplicatedObjectId& Object)
	{
		const FClientId& ClientId = Object.SenderEndpointId;
		FClientAuthorityData* ClientData = ClientAuthorityData.Find(ClientId);
		if (!ClientData)
		{
			return;
		}

		const FStreamId& StreamId = Object.StreamId;
		TSet<FSoftObjectPath>* AuthoredObjects = ClientData->OwnedObjects.Find(StreamId);
		if (!AuthoredObjects)
		{
			return;
		}

		// This is all that is needed to remove authority.
		AuthoredObjects->Remove(Object.Object);
		
		// Clean-up potentially empty entries
		if (AuthoredObjects->IsEmpty())
		{
			ClientAuthorityData.Remove(StreamId);
			if (ClientData->OwnedObjects.IsEmpty())
			{
				ClientAuthorityData.Remove(ClientId);
			}
		}
	}


	EConcertSessionResponseCode FAuthorityManager::HandleChangeAuthorityRequest(
		const FConcertSessionContext& Context,
		const FConcertReplication_ChangeAuthority_Request& Request,
		FConcertReplication_ChangeAuthority_Response& Response
		)
	{
		// This log does two things: 1. Identify issues in unit tests / at runtime 2. Warn about possibly malicious attempts when the server runs.
		UE_CLOGF(Request.TakeAuthority.IsEmpty() && Request.ReleaseAuthority.IsEmpty(), LogConcert, Warning, "Received invalid authority request (TakeAuthority.Num() == 0 and ReleaseAuthority.Num() == 0)");
		
		const FGuid ClientId = Context.SourceEndpointId;
		LogNetworkMessage(CVarLogAuthorityRequestsAndResponsesOnServer, Request, [&](){ return GetClientName(*Session, ClientId); });
		InternalApplyChangeAuthorityRequest(Context.SourceEndpointId, Request, Response.RejectedObjects, Response.SyncControl);
		LogNetworkMessage(CVarLogAuthorityRequestsAndResponsesOnServer, Response, [&](){ return GetClientName(*Session, ClientId); });
		
		Response.ErrorCode = EReplicationResponseErrorCode::Handled;
		return EConcertSessionResponseCode::Success;
	}
	
	void FAuthorityManager::InternalApplyChangeAuthorityRequest(
		const FClientId& EndpointId,
		const FConcertReplication_ChangeAuthority_Request& Request,
		TMap<FSoftObjectPath, FConcertStreamArray>& OutRejectedObjects,
		FConcertReplication_ChangeSyncControl& OutChangedSyncControl,
		bool bShouldLog
		)
	{
		bool bMadeChanges = false;
		
		const FClientId& ClientId = EndpointId;
		FClientAuthorityData& AuthorityData = ClientAuthorityData.FindOrAdd(ClientId);
		Private::ForEachReplicatedObject(Request.TakeAuthority, [this, &OutRejectedObjects, bShouldLog, &bMadeChanges, &AuthorityData, &ClientId](const FStreamId& StreamId, const FSoftObjectPath& ObjectPath)
		{
			const FConcertReplicatedObjectId ObjectToAuthor{ { StreamId, ObjectPath }, ClientId };
			if (CanTakeAuthority(ObjectToAuthor))
			{
				UE_CLOGF(bShouldLog, LogConcert, Log, "Transferred authority of %ls to client %ls for their stream %ls", *ObjectPath.ToString(), *ClientId.ToString(EGuidFormats::Short), *StreamId.ToString(EGuidFormats::Short));
				AuthorityData.OwnedObjects.FindOrAdd(StreamId).Add(ObjectPath);
				bMadeChanges = true;
			}
			else
			{
				UE_CLOGF(bShouldLog, LogConcert, Log, "Rejected %ls request of authority over %ls in stream %ls", *ClientId.ToString(EGuidFormats::Short), *ObjectPath.ToString(), *StreamId.ToString(EGuidFormats::Short));
				OutRejectedObjects.FindOrAdd(ObjectPath).StreamIds.AddUnique(StreamId);
			}
		});
		
		Private::ForEachReplicatedObject(Request.ReleaseAuthority, [this, &bMadeChanges, &AuthorityData](const FStreamId& StreamId, const FSoftObjectPath& ObjectPath)
		{
			TSet<FSoftObjectPath>* OwnedObjects = AuthorityData.OwnedObjects.Find(StreamId);
			// Though dubious, it is a valid request for the client to release non-owned objects
			if (!OwnedObjects)
			{
				return;
			}

			OwnedObjects->Remove(ObjectPath);
			// Avoid memory leaking
			if (OwnedObjects->IsEmpty())
			{
				AuthorityData.OwnedObjects.Remove(StreamId);
				bMadeChanges = true;
			}
		});

		if (bMadeChanges && ensure(GenerateSyncControlDelegate.IsBound()))
		{
			OutChangedSyncControl = GenerateSyncControlDelegate.Execute(ClientId);
		}
	}
}
