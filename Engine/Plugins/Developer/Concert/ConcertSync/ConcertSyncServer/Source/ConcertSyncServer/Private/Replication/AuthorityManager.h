// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessages.h"
#include "Enumeration/IRegistrationEnumerator.h"
#include "Replication/Data/ObjectIds.h"

#include "Delegates/Delegate.h"
#include "Misc/EBreakBehavior.h"
#include "Templates/Function.h"

struct FConcertReplicationStreamArray;
struct FConcertStreamArray;
struct FConcertReplication_ChangeSyncControl;
class IConcertSession;

struct FConcertReplication_ChangeAuthority_Response;
struct FConcertReplication_ChangeAuthority_Request;
struct FConcertPropertyChain;
struct FConcertPropertySelection;
struct FConcertSessionContext;
struct FConcertObjectInStreamID;
struct FConcertObjectInStreamArray;
struct FConcertReplicatedObjectId;
struct FConcertReplicationStream;

namespace UE::ConcertSyncServer::Replication
{
	/** Responds to FConcertChangeAuthority_Request and tracks what objects and properties clients have authority over. */
	class FAuthorityManager : public FNoncopyable
	{
	public:
		
		using FStreamId = FGuid;
		using FClientId = FGuid;
		using FProcessAuthorityConflict = TFunctionRef<EBreakBehavior(const FClientId& ClientId, const FStreamId& StreamId, const FConcertPropertyChain& WrittenProperties)>;
		
		/**
		 * @param InGetters Gets information about clients' registered streams 
		 * @param InSession The session to handle authority requests on
		 * @param InGenerateSyncControlDelegate Called to fill the sync control portion of FConcertReplication_ChangeAuthority_Response.
		 */
		FAuthorityManager(IRegistrationEnumerator& InGetters UE_LIFETIMEBOUND, TSharedRef<IConcertSession> InSession);
		~FAuthorityManager();

		/**
		 * Checks whether the client that sent the identified object had authority to send it.
		 * @return Whether the server should process the object change.
		 */
		bool HasAuthorityToChange(const FConcertReplicatedObjectId& ObjectChange) const;

		/** Utility for iterating authority a client has for a given stream. */
		void EnumerateAuthority(const FClientId& ClientId, const FStreamId& StreamId, TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object)> Callback) const;
		/** Util for getting all client owned objects as array. */
		TArray<FConcertObjectInStreamID> GetOwnedObjects(const FClientId& ClientId) const;

		enum class EAuthorityResult : uint8
		{
			/** The client is allowed to take authority */
			Allowed,
			/** There was at least one conflict */
			Conflict,
			/** No conflicts were checked because OverwriteProperties was nullptr and the client had not registered any properties to send for the given object. */
			NoRegisteredProperties
		};
		/**
		 * Enumerates all authority conflicts, if any, that would occur if ObjectChange.SenderEndpointId were to take authority over the identified object.
		 * You can optionally supply OverwriteProperties, which is useful if you're about to change the contents of the stream.
		 * 
		 * @param Object Identifies the object, its stream and client that will send the object. Conflicts are checked against this.
		 * @param OverrideProperties Optional properties that Object will have and conflicts should be checked against. If not overriden, falls back to the properties registered thus far.
		 * @param ProcessConflict Invoked for the potential conflicts
		 * @return Whether there were any conflicts
		 */
		EAuthorityResult EnumerateAuthorityConflicts(
			const FConcertReplicatedObjectId& Object,
			const FConcertPropertySelection* OverrideProperties = nullptr,
			FProcessAuthorityConflict ProcessConflict = [](const FClientId&, const FStreamId&, const FConcertPropertyChain&){ return EBreakBehavior::Break; }
			) const;
		/** Whether it is legal for this the client identified by ClientId to take control over the object given the stream the client has registered. */
		bool CanTakeAuthority(const FConcertReplicatedObjectId& Object) const;

		/**
		 * Enumerates all authority conflicts, if any, that would occur if ObjectChange.SenderEndpointId were to take authority over the identified object.
		 * This version allows you to override the entire session content (compared to EnumerateAuthorityConflicts, which only allows you to specify OverwriteProperties). 
		 * 
		 * @param Object Identifies the object, its stream and client that will send the object. Conflicts are checked against this.
		 * @param StreamOverrides Overrides that the stream content of connected clients. If a client is not overriden, falls back to the content registered thus far.
		 *		The key can be non-existing clients if you want to validate no conflicts were to occur if some other clients were present("injection").
		 * @param AuthorityOverrides Overrides the authority of connected and "injected" clients. If a client is not overriden, falls back to the authority registered thus far.
		 * @param ProcessConflict Invoked for the potential conflicts
		 * 
		 * @return Whether there were any conflicts
		 */
		EAuthorityResult EnumerateAuthorityConflictsWithOverrides(
			const FConcertReplicatedObjectId& Object,
			const TMap<FGuid, FConcertReplicationStreamArray>& StreamOverrides,
			const TMap<FGuid, FConcertObjectInStreamArray>& AuthorityOverrides,
			FProcessAuthorityConflict ProcessConflict = [](const FClientId&, const FStreamId&, const FConcertPropertyChain&){ return EBreakBehavior::Break; }
			) const;

		/** Notifies this manager that the client has left, which means all their authority is now gone. */
		void OnPostClientLeft(const FClientId& ClientEndpointId);
		/** Takes away authority from the given client from the given object. */
		void RemoveAuthority(const FConcertReplicatedObjectId& Object);

		/** Applies a change authority request as if EndpointId had sent it. */
		void ApplyChangeAuthorityRequest(const FClientId& EndpointId, const FConcertReplication_ChangeAuthority_Request& Request, TMap<FSoftObjectPath, FConcertStreamArray>& OutRejectedObjects, FConcertReplication_ChangeSyncControl& OutChangedSyncControl)
		{
			InternalApplyChangeAuthorityRequest(EndpointId, Request, OutRejectedObjects,  OutChangedSyncControl, false);
		}

		DECLARE_DELEGATE_RetVal_OneParam(FConcertReplication_ChangeSyncControl, FGenerateSyncControl, const FGuid& ClientId);
		/** Sets the delegate that is called to fill the sync control portion of FConcertReplication_ChangeAuthority_Response. */
		FGenerateSyncControl& OnGenerateSyncControl() { return GenerateSyncControlDelegate; }
		
	private:

		/** Callbacks required to obtain client info. */
		IRegistrationEnumerator& Getters;
		/** The session under which this manager operates. */
		TSharedRef<IConcertSession> Session;

		struct FClientAuthorityData
		{
			/** Objects the client has authority over. */
			TMap<FStreamId, TSet<FSoftObjectPath>> OwnedObjects;
		};
		TMap<FClientId, FClientAuthorityData> ClientAuthorityData;

		/** Called to fill the sync control portion of FConcertReplication_ChangeAuthority_Response. */
		FGenerateSyncControl GenerateSyncControlDelegate;

		EConcertSessionResponseCode HandleChangeAuthorityRequest(
			const FConcertSessionContext& Context,
			const FConcertReplication_ChangeAuthority_Request& Request,
			FConcertReplication_ChangeAuthority_Response& Response
			);
		void InternalApplyChangeAuthorityRequest(
			const FClientId& EndpointId,
			const FConcertReplication_ChangeAuthority_Request& Request,
			TMap<FSoftObjectPath, FConcertStreamArray>& OutRejectedObjects,
			FConcertReplication_ChangeSyncControl& OutChangedSyncControl,
			bool bShouldLog = true
			);
	};
}

