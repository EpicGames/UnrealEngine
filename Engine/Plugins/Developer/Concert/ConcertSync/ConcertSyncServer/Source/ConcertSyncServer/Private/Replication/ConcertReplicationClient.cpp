// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertReplicationClient.h"

#include "Processing/ServerReplicationDataQueuer.h"
#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/Messages/Handshake.h"

namespace UE::ConcertSyncServer::Replication
{
	FConcertReplicationClient::FConcertReplicationClient(
		TArray<FConcertReplicationStream> StreamDescriptions,
		const FGuid& ClientEndpointId,
		IConcertSession& Session,
		ConcertSyncCore::FObjectReplicationCache& ReplicationCache,
		ConcertSyncCore::FGetObjectFrequencySettings GetObjectFrequencySettingsDelegate
	)
		: StreamDescriptions(MoveTemp(StreamDescriptions))
		, ClientEndpointId(ClientEndpointId)
		, EventQueue(FServerReplicationDataQueuer::Make(ClientEndpointId, ReplicationCache))
		, DataRelay(
			MoveTemp(GetObjectFrequencySettingsDelegate),
			ClientEndpointId, Session, *EventQueue
			)
	{}

	void FConcertReplicationClient::ProcessClient(const ConcertSyncCore::FProcessObjectsParams& Params)
	{
		DataRelay.ProcessObjects(Params);
	}

	void FConcertReplicationClient::ApplyValidatedRequest(const FConcertReplication_ChangeStream_Request& Request)
	{
		// Right now there is nothing further to do but if in future you need to update some client systems of the change, this is the place to do it.
		ConcertSyncCore::Replication::ChangeStreamUtils::ApplyValidatedRequest(Request, StreamDescriptions);
	}
}
