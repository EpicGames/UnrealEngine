// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Replication/Processing/ObjectReplicationSender.h"
#include "Replication/Processing/Proxy/ObjectProcessorProxy_Frequency.h"
#include "Templates/SharedPointer.h"

struct FConcertReplicationStream;
struct FConcertReplication_Join_Request;
struct FConcertReplication_ChangeStream_Request;

namespace UE::ConcertSyncCore
{
	class FObjectReplicationCache;
	struct FProcessObjectsParams;
}

namespace UE::ConcertSyncServer::Replication
{
	class FServerReplicationDataQueuer;

	/** Server-side representation of a remote replication client. */
	class FConcertReplicationClient : public FNoncopyable
	{
	public:

		FConcertReplicationClient(
			TArray<FConcertReplicationStream> StreamDescriptions,
			const FGuid& ClientEndpointId,
			IConcertSession& Session UE_LIFETIMEBOUND,
			ConcertSyncCore::FObjectReplicationCache& ReplicationCache UE_LIFETIMEBOUND,
			ConcertSyncCore::FGetObjectFrequencySettings GetObjectFrequencySettingsDelegate
			);

		/**
		 * Process any latent tasks, such as processing pending events that need to be sent to the remote instance.
		 * Process given a time budget. The time budget may be exceeded but we'll try not to and to stay as close to the budget as possible.
		 */
		void ProcessClient(const ConcertSyncCore::FProcessObjectsParams& Params);
		
		/** Updates the StreamDescriptions array with the changes from Request. The request already passed validation and is valid to apply. */
		void ApplyValidatedRequest(const FConcertReplication_ChangeStream_Request& Request);
		
		const FGuid& GetClientEndpointId() const { return ClientEndpointId; }
		const TArray<FConcertReplicationStream>& GetStreamDescriptions() const { return StreamDescriptions; }

	private:

		/** The streams this client offered to send. */
		TArray<FConcertReplicationStream> StreamDescriptions;
		
		/** This client's endpoint ID. */
		const FGuid ClientEndpointId;
		
		/** Queues up replication data and passes it to DataRelay. */
		TSharedRef<FServerReplicationDataQueuer> EventQueue;

		/** Sends to remote endpoint and makes sure the objects are replicated at the specified frequency settings. */
		using FDataRelayThrottledByFrequency = ConcertSyncCore::TObjectProcessorProxy_Frequency<ConcertSyncCore::FObjectReplicationSender>;
		/** Sends data to the remote endpoint */
		FDataRelayThrottledByFrequency DataRelay;
	};
}
