// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerReplicationManager.h"

#include "AuthorityManager.h"
#include "ConcertLogGlobal.h"
#include "ConcertServerWorkspace.h"
#include "IConcertSession.h"
#include "Replication/ConcertReplicationClient.h"
#include "Replication/Data/ClientQueriedInfo.h"
#include "Replication/Formats/FullObjectFormat.h"
#include "Replication/IReplicationWorkspace.h"
#include "Replication/Messages/ChangeStream.h"
#include "Replication/Messages/ClientQuery.h"
#include "Replication/Messages/Handshake.h"
#include "Replication/Messages/PutState.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Replication/Messages/RestoreContent.h"
#include "Replication/Processing/ObjectReplicationCache.h"
#include "Util/JoinRequestValidation.h"
#include "Util/LogUtils.h"
#include "Util/ReplicationCVars.h"

namespace UE::ConcertSyncServer::Replication
{
	FConcertServerReplicationManager::FConcertServerReplicationManager(
		TSharedRef<IConcertServerSession> InLiveSession,
		IReplicationWorkspace& InServerWorkspace,
		EConcertSyncSessionFlags InSessionFlags
		)
		: Session(InLiveSession)
		, ServerWorkspace(InServerWorkspace)
		, SessionFlags(InSessionFlags)
		, ReplicationFormat(MakeUnique<ConcertSyncCore::FFullObjectFormat>())
		, AuthorityManager(*this, Session)
		, MuteManager(*Session, ServerObjectCache, SessionFlags)
		, SyncControlManager(*Session, AuthorityManager, MuteManager, *this)
		, ReplicationCache(MakeShared<ConcertSyncCore::FObjectReplicationCache>(*ReplicationFormat))
		, ReplicationDataReceiver(AuthorityManager, SyncControlManager, *Session, *ReplicationCache)
	{
		Session->RegisterCustomRequestHandler<FConcertReplication_Join_Request, FConcertReplication_Join_Response>(this, &FConcertServerReplicationManager::HandleJoinReplicationSessionRequest);
		Session->RegisterCustomRequestHandler<FConcertReplication_QueryReplicationInfo_Request, FConcertReplication_QueryReplicationInfo_Response>(this, &FConcertServerReplicationManager::HandleQueryReplicationInfoRequest);
		Session->RegisterCustomRequestHandler<FConcertReplication_ChangeStream_Request, FConcertReplication_ChangeStream_Response>(this, &FConcertServerReplicationManager::HandleChangeStreamRequest);
		Session->RegisterCustomRequestHandler<FConcertReplication_RestoreContent_Request, FConcertReplication_RestoreContent_Response>(this, &FConcertServerReplicationManager::HandleRestoreContentRequest);
		Session->RegisterCustomRequestHandler<FConcertReplication_PutState_Request, FConcertReplication_PutState_Response>(this, &FConcertServerReplicationManager::HandlePutStateRequest);
		Session->RegisterCustomEventHandler<FConcertReplication_LeaveEvent>(this, &FConcertServerReplicationManager::HandleLeaveReplicationSessionRequest);
		
		Session->OnSessionClientChanged().AddRaw(this, &FConcertServerReplicationManager::OnConnectionChanged);
		Session->OnTick().AddRaw(this, &FConcertServerReplicationManager::Tick);

		MuteManager.OnMuteRequestApplied().AddRaw(this, &FConcertServerReplicationManager::GenerateMuteActivity);
	}

	FConcertServerReplicationManager::~FConcertServerReplicationManager()
	{
		Session->UnregisterCustomRequestHandler<FConcertReplication_Join_Response>();
		Session->UnregisterCustomRequestHandler<FConcertReplication_QueryReplicationInfo_Response>();
		Session->UnregisterCustomEventHandler<FConcertReplication_LeaveEvent>(this);

		Session->OnTick().RemoveAll(this);
	}

	void FConcertServerReplicationManager::ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback) const
	{
		const TUniquePtr<FConcertReplicationClient>* Client = Clients.Find(ClientEndpointId);
		if (!ensure(Client))
		{
			return;
		}

		for (const FConcertReplicationStream& Stream : (*Client)->GetStreamDescriptions())
		{
			if (Callback(Stream) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	void FConcertServerReplicationManager::ForEachReplicationClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const
	{
		for (const TPair<FGuid, TUniquePtr<FConcertReplicationClient>>& ClientPair : Clients)
		{
			if (Callback(ClientPair.Key) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	EConcertSessionResponseCode FConcertServerReplicationManager::HandleJoinReplicationSessionRequest(
		const FConcertSessionContext& ConcertSessionContext,
		const FConcertReplication_Join_Request& Request,
		FConcertReplication_Join_Response& Response
		)
	{
		const FGuid ClientId = ConcertSessionContext.SourceEndpointId;
		// Have a pair of logs before and after processing in case of potential disaster
		UE_LOGF(LogConcert, Log, "Received replication join request from endpoint %ls", *ClientId.ToString());
		
		LogNetworkMessage(CVarLogAuthorityRequestsAndResponsesOnServer, Request, [&](){ return GetClientName(*Session, ClientId); });
		const EConcertSessionResponseCode Result = InternalHandleJoinReplicationSessionRequest(ConcertSessionContext, Request, Response);
		LogNetworkMessage(CVarLogAuthorityRequestsAndResponsesOnServer, Response, [&](){ return GetClientName(*Session, ClientId); });

		const bool bSuccess = Response.JoinErrorCode == EJoinReplicationErrorCode::Success;
		if (bSuccess)
		{
			Response.SyncControl = SyncControlManager.OnGenerateSyncControlForClientJoin(ClientId);
			ServerObjectCache.OnJoin(ClientId, Request);
		}
		
		UE_CLOGF(bSuccess, LogConcert, Log, "Accepted replication join request");
		UE_CLOGF(!bSuccess, LogConcert, Log, "Rejected replication join request. %ls: %ls", *ConcertSyncCore::Replication::LexJoinErrorCode(Response.JoinErrorCode), *Response.DetailedErrorMessage);
		return Result;
	}

	EConcertSessionResponseCode FConcertServerReplicationManager::InternalHandleJoinReplicationSessionRequest(
		const FConcertSessionContext& ConcertSessionContext,
		const FConcertReplication_Join_Request& Request,
		FConcertReplication_Join_Response& Response
		)
	{
		const FGuid& ClientId = ConcertSessionContext.SourceEndpointId;
		const bool bHasClient = Session->GetSessionClientEndpointIds().Contains(ClientId);
		if (!bHasClient)
		{
			Response = { EJoinReplicationErrorCode::NotInAnyConcertSession, TEXT("Client must be in a Concert Session!") };
			return EConcertSessionResponseCode::Success;
		}

		if (Clients.Contains(ClientId))
		{
			Response = { EJoinReplicationErrorCode::AlreadyInSession };
			return EConcertSessionResponseCode::Success;
		}

		auto[ErrorCode, ErrorMessage, StreamDescriptions] = ValidateRequest(Request);
		if (ErrorCode != EJoinReplicationErrorCode::Success)
		{
			Response = { ErrorCode, ErrorMessage };
			return EConcertSessionResponseCode::Success;
		}

		Clients.Emplace(
			ClientId,
			MakeUnique<FConcertReplicationClient>(
				MoveTemp(StreamDescriptions),
				ClientId,
				*Session,
				*ReplicationCache,
				ConcertSyncCore::FGetObjectFrequencySettings::CreateRaw(this, &FConcertServerReplicationManager::GetObjectFrequencySettings)
			)
		);
		Response = { EJoinReplicationErrorCode::Success };
		return EConcertSessionResponseCode::Success;
	}

	EConcertSessionResponseCode FConcertServerReplicationManager::HandleQueryReplicationInfoRequest(
		const FConcertSessionContext&,
		const FConcertReplication_QueryReplicationInfo_Request& Request,
		FConcertReplication_QueryReplicationInfo_Response& Response
		)
	{
		for (const FGuid& EndpointId : Request.ClientEndpointIds)
		{
			const TUniquePtr<FConcertReplicationClient>* Client = Clients.Find(EndpointId);
			if (!Client)
			{
				// This could happen if the client left the replication session before this request was answered
				continue;
			}
			
			FConcertQueriedClientInfo& EndpointInfo = Response.ClientInfo.Add(EndpointId);
			if (!EnumHasAnyFlags(Request.QueryFlags, EConcertQueryClientStreamFlags::SkipStreamInfo))
			{
				EndpointInfo.Streams = BuildClientStreamInfo(*Client->Get(), Request.QueryFlags);
			}
			if (!EnumHasAnyFlags(Request.QueryFlags, EConcertQueryClientStreamFlags::SkipAuthority))
			{
				EndpointInfo.Authority = BuildClientAuthorityInfo(*Client->Get());
			}
			
		}

		Response.ErrorCode = EReplicationResponseErrorCode::Handled;
		return EConcertSessionResponseCode::Success;
	}

	TArray<FConcertBaseStreamInfo> FConcertServerReplicationManager::BuildClientStreamInfo(const FConcertReplicationClient& Client, EConcertQueryClientStreamFlags QueryFlags)
	{
		TArray<FConcertBaseStreamInfo> Result;
		Algo::Transform(Client.GetStreamDescriptions(), Result, [QueryFlags](const FConcertReplicationStream& Description)
		{
			using namespace ConcertSyncCore;
			
			EReplicationStreamCloneFlags Flags = EReplicationStreamCloneFlags::None;
			if (EnumHasAnyFlags(QueryFlags, EConcertQueryClientStreamFlags::SkipProperties))
			{
				Flags |= EReplicationStreamCloneFlags::SkipProperties;
			}
			if (EnumHasAnyFlags(QueryFlags, EConcertQueryClientStreamFlags::SkipFrequency))
			{
				Flags |= EReplicationStreamCloneFlags::SkipFrequency;
			}
			
			return Description.BaseDescription.Clone(Flags);
		});
		return Result;
	}

	TArray<FConcertAuthorityClientInfo> FConcertServerReplicationManager::BuildClientAuthorityInfo(const FConcertReplicationClient& Client) const
	{
		TArray<FConcertAuthorityClientInfo> Result;
		for (const FConcertReplicationStream& Description : Client.GetStreamDescriptions())
		{
			const FGuid StreamId = Description.BaseDescription.Identifier;
			FConcertAuthorityClientInfo Info;
			Info.StreamId = StreamId;
			
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : Description.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				FConcertReplicatedObjectId ObjectInfo;
				ObjectInfo.SenderEndpointId = Client.GetClientEndpointId();
				ObjectInfo.Object = Pair.Key;
				ObjectInfo.StreamId = StreamId;
				
				if (AuthorityManager.HasAuthorityToChange(ObjectInfo))
				{
					Info.AuthoredObjects.Add(Pair.Key);
				}
			}

			// There is no point in sending empty data
			if (!Info.AuthoredObjects.IsEmpty())
			{
				Result.Add(Info);
			}
		}
		return Result;
	}

	void FConcertServerReplicationManager::HandleLeaveReplicationSessionRequest(
		const FConcertSessionContext& ConcertSessionContext,
		const FConcertReplication_LeaveEvent& EventData)
	{
		const FGuid ClientEndpointId = ConcertSessionContext.SourceEndpointId;
		UE_LOGF(LogConcert, Log, "Received replication leave request from endpoint %ls", *ClientEndpointId.ToString());
		
		OnClientLeftReplication(ClientEndpointId);
	}

	void FConcertServerReplicationManager::OnConnectionChanged(IConcertServerSession& ConcertServerSession, EConcertClientStatus ConcertClientStatus, const FConcertSessionClientInfo& ClientInfo)
	{
		const FGuid ClientEndpointId = ClientInfo.ClientEndpointId;
		if (ConcertClientStatus == EConcertClientStatus::Disconnected)
		{
			OnClientLeftReplication(ClientEndpointId);
		}
	}

	void FConcertServerReplicationManager::OnClientLeftReplication(const FGuid& EndpointId)
	{
		if (!Clients.Contains(EndpointId))
		{
			return;
		}
		
		TUniquePtr<FConcertReplicationClient> RemovedClient;
		const bool bRemoved = Clients.RemoveAndCopyValue(EndpointId, RemovedClient);
		check(bRemoved);

		ProduceClientLeftActivity(*RemovedClient);

		// ServerObjectCache should be updated before anyone else that may rely on its state.
		ServerObjectCache.OnPostClientLeft(EndpointId, RemovedClient->GetStreamDescriptions());
		
		// There is some inefficiency here: FMuteManager::OnMuteStateChanged may broadcast, which causes SyncControlManager to rebuild ...
		MuteManager.OnPostClientLeft(RemovedClient->GetStreamDescriptions());
		AuthorityManager.OnPostClientLeft(EndpointId);
		
		// ... and then the sync control manager rebuilds again.
		SyncControlManager.OnPostClientLeft(EndpointId);
	}

	void FConcertServerReplicationManager::ProduceClientLeftActivity(const FConcertReplicationClient& Client) const
	{
		if (EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldEnableReplicationActivities))
		{
			const FGuid& EndpointId = Client.GetClientEndpointId();
			
			FConcertSyncReplicationPayload_LeaveReplication LeaveReplication;
			LeaveReplication.Streams = Client.GetStreamDescriptions();
			LeaveReplication.OwnedObjects = AuthorityManager.GetOwnedObjects(EndpointId);
			ServerWorkspace.ProduceClientLeaveReplicationActivity(EndpointId, LeaveReplication);
		}
	}

	void FConcertServerReplicationManager::GenerateMuteActivity(const FGuid& EndpointId, const FConcertReplication_ChangeMuteState_Request& Request) const
	{
		if (EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldEnableReplicationActivities))
		{
			const FConcertSyncReplicationPayload_Mute MutePayload { Request };
			ServerWorkspace.ProduceClientMuteReplicationActivity(EndpointId, MutePayload);
		}
	}

	void FConcertServerReplicationManager::Tick(IConcertServerSession& InSession, float InDeltaTime)
	{
		// TODO UE-190714: We may want to set a time budget for clients.
		// TODO UE-203340: Time slice clients
		for (TPair<FGuid, TUniquePtr<FConcertReplicationClient>>& Client : Clients)
		{
			Client.Value->ProcessClient({ InDeltaTime });
		}
	}

	FConcertObjectReplicationSettings FConcertServerReplicationManager::GetObjectFrequencySettings(const FConcertReplicatedObjectId& Object) const
	{
		const TUniquePtr<FConcertReplicationClient>* Client = Clients.Find(Object.SenderEndpointId);
		if (!Client)
		{
			UE_LOGF(LogConcert, Warning, "Requested frequency settings for unknown client %ls", *Object.SenderEndpointId.ToString());
			return {};
		}
		
		const FConcertReplicationStream* Stream = Client->Get()->GetStreamDescriptions().FindByPredicate([&Object](const FConcertReplicationStream& Description)
			{
				return Description.BaseDescription.Identifier == Object.StreamId;
			});
		if (!Stream)
		{
			UE_LOGF(LogConcert, Warning, "Requested frequency settings for unknown stream %ls and object %ls", *Object.StreamId.ToString(), *Object.Object.ToString());
			return {};
		}
			
		return Stream->BaseDescription.FrequencySettings.GetSettingsFor(Object.Object);
	}
}

