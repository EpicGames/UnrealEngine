// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AuthorityManager.h"
#include "ConcertMessages.h"
#include "ConcertReplicationClient.h"
#include "Enumeration/IRegistrationEnumerator.h"
#include "Muting/MuteManager.h"
#include "Replication/Formats/IObjectReplicationFormat.h"
#include "Replication/IConcertServerReplicationManager.h"
#include "Replication/Messages/Handshake.h"
#include "Replication/Misc/ReplicatedObjectHierarchyCache.h"
#include "Replication/Processing/ObjectReplicationCache.h"
#include "Replication/Processing/ServerObjectReplicationReceiver.h"
#include "SyncControlManager.h"

#include "HAL/Platform.h"
#include "Replication/Messages/ChangeClientEvent.h"
#include "Replication/Messages/PutState.h"
#include "Replication/Messages/RestoreContent.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

struct FConcertReplication_ChangeClientEvent;
class FConcertServerWorkspace;
class IConcertClientReplicationBridge;
class IConcertServerSession;

enum class EConcertSyncSessionFlags : uint32;
enum class EConcertQueryClientStreamFlags : uint8;

struct FConcertAuthorityClientInfo;
struct FConcertReplication_ChangeMuteState_Request;
struct FConcertReplication_ChangeStream_Response;
struct FConcertReplication_QueryReplicationInfo_Response;
struct FConcertReplication_QueryReplicationInfo_Request;
struct FConcertSyncReplicationPayload_LeaveReplication;

namespace UE::ConcertSyncCore
{
	class IObjectReplicationFormat;
	class FObjectReplicationCache;
}

namespace UE::ConcertSyncServer::Replication
{
	class FAuthorityManager;
	class IReplicationWorkspace;

	/**
	 * Manages all server-side systems relevant to the Replication features.
	 * 
	 * Primarily responds to client requests to join (handshake) and leave replication delegating the result of the
	 * operation to relevant systems. 
	 */
	class FConcertServerReplicationManager
		: public IConcertServerReplicationManager
		, public IRegistrationEnumerator
		, public FNoncopyable
	{
	public:
		
		explicit FConcertServerReplicationManager(TSharedRef<IConcertServerSession> InLiveSession, IReplicationWorkspace& InServerWorkspace UE_LIFETIMEBOUND, EConcertSyncSessionFlags InSessionFlags);
		virtual ~FConcertServerReplicationManager() override;

		const FAuthorityManager& GetAuthorityManager() const { return AuthorityManager; }

		//~ Begin IStreamEnumerator Interface
		virtual void ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback) const override;
		//~ End IStreamEnumerator Interface
		
		//~ Begin IClientEnumerator Interface
		virtual void ForEachReplicationClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const override;
		//~ End IClientEnumerator Interface

	private:
		
		/** Session instance this manager was created for. */
		const TSharedRef<IConcertServerSession> Session;
		/** Used to produce replication activities. */
		IReplicationWorkspace& ServerWorkspace;
		/** Used to determine which dynamic features are enabled. */
		const EConcertSyncSessionFlags SessionFlags;
		
		/** Responsible for analysing received replication data. */
		TUniquePtr<ConcertSyncCore::IObjectReplicationFormat> ReplicationFormat;

		/**
		 * Holds the outer hierarchy of all objects registered in any stream.
		 * 
		 * This receives join, leave, and change stream events.
		 * The events are processed after the mute manager does.
		 */
		ConcertSyncCore::FReplicatedObjectHierarchyCache ServerObjectCache;

		/** Responds to client requests to changing authority and can be asked whether an object change is valid to take place. */
		FAuthorityManager AuthorityManager;
		/** Responds to client mute requests and stores the mute states. */
		FMuteManager MuteManager;
		/** Decides whether clients should be replicating. Clients may replicate when they have authority and there are other clients listening for that data. */
		FSyncControlManager SyncControlManager;
		
		/** Received replication events are put into the ReplicationCache. The cache is used to relay data to clients latently. */
		const TSharedRef<ConcertSyncCore::FObjectReplicationCache> ReplicationCache;
		/** Receives replication events from all endpoints. */
		FServerObjectReplicationReceiver ReplicationDataReceiver;

		/**
		 * Clients that have requested to join replication. Maps client ID to replication info.
		 * Clients are stored in a unique ptr to avoid dealing with reallocations when the map resizes.
		 */
		TMap<FGuid, TUniquePtr<FConcertReplicationClient>> Clients;

		// Joining
		EConcertSessionResponseCode HandleJoinReplicationSessionRequest(const FConcertSessionContext& ConcertSessionContext, const FConcertReplication_Join_Request& Request, FConcertReplication_Join_Response& Response);
		EConcertSessionResponseCode InternalHandleJoinReplicationSessionRequest(const FConcertSessionContext& ConcertSessionContext, const FConcertReplication_Join_Request& Request, FConcertReplication_Join_Response& Response);

		// Querying
		EConcertSessionResponseCode HandleQueryReplicationInfoRequest(const FConcertSessionContext& ConcertSessionContext, const FConcertReplication_QueryReplicationInfo_Request& Request, FConcertReplication_QueryReplicationInfo_Response& Response);
		/** Gets all registered streams and optionally removes the properties. */
		static TArray<FConcertBaseStreamInfo> BuildClientStreamInfo(const FConcertReplicationClient& Client, EConcertQueryClientStreamFlags QueryFlags);
		/** Maps the client's streams to the objects in that stream the client has taken authority over. */
		TArray<FConcertAuthorityClientInfo> BuildClientAuthorityInfo(const FConcertReplicationClient& Client) const;

		// Changing streams
		EConcertSessionResponseCode HandleChangeStreamRequest(const FConcertSessionContext& ConcertSessionContext, const FConcertReplication_ChangeStream_Request& Request, FConcertReplication_ChangeStream_Response& Response);
		/** Applies a validated change stream request. */
		void ApplyChangeStreamRequest(const FConcertReplication_ChangeStream_Request& Request, FConcertReplicationClient& Client);

		// Restoring content
		EConcertSessionResponseCode HandleRestoreContentRequest(const FConcertSessionContext& ConcertSessionContext, const FConcertReplication_RestoreContent_Request& Request, FConcertReplication_RestoreContent_Response& Response);
		/** @return Response code to return for the request and the payload to apply, if there is anything that should be applied. */
		TTuple<EConcertReplicationRestoreErrorCode, TOptional<FConcertSyncReplicationPayload_LeaveReplication>> ValidateRestoreContentRequest(const FGuid& RequestingEndpointId, const FConcertReplication_RestoreContent_Request& Request) const;
		/** Applies a validated restore content request. */
		void ApplyRestoreContentRequest(const FGuid& RequestingEndpointId, const FConcertReplication_RestoreContent_Request& Request, const FConcertSyncReplicationPayload_LeaveReplication& DataToApply, FConcertReplication_ChangeSyncControl& ChangedSyncControl);
		/** Restores the stream content in DataToApply to Client according to how Request specifies it. */
		void RestoreStreamContent(const FConcertReplication_RestoreContent_Request& Request, const FConcertSyncReplicationPayload_LeaveReplication& DataToApply, FConcertReplicationClient& Client, FConcertReplication_ChangeSyncControl& OutChangedSyncControl);
		/** Restores the authority in DataToApply to Client. */
		void RestoreAuthority(const FConcertSyncReplicationPayload_LeaveReplication& DataToApply, const FConcertReplicationClient& Client, FConcertReplication_ChangeSyncControl& OutChangedSyncControl);
		
		/** Restores the mute state of Client. */
		void RestoreMuteState(const FConcertReplicationClient& Client, FConcertReplication_ChangeSyncControl& OutChangedSyncControl);
		void ApplyRestoringMuteRequest(const FConcertReplicationClient& Client, const FConcertReplication_ChangeMuteState_Request& AggregatedRequest, FConcertReplication_ChangeSyncControl& OutChangedSyncControl);

		// Changing multiple clients in one go
		EConcertSessionResponseCode HandlePutStateRequest(const FConcertSessionContext& Context, const FConcertReplication_PutState_Request& Request, FConcertReplication_PutState_Response& Response);
		void ApplyPutStateRequest(
			const FGuid& RequestingEndpointId,
			const FConcertReplication_PutState_Request& Request,
			const TMap<FGuid, FConcertReplication_ChangeStream_Request> StreamRequests,
			FConcertReplication_PutState_Response& Response
			);
		void ApplyPutState_Streams(
			const FGuid& RequestingEndpointId,
			const TMap<FGuid, FConcertReplication_ChangeStream_Request> StreamRequests,
			TMap<FGuid, FConcertReplication_ClientChangeData>& ClientChanges
		);
		void ApplyPutState_Authority(
			const FGuid& RequestingEndpointId,
			const TSet<FConcertObjectInStreamID> RequestingClientSyncControlBefore,
			const FConcertReplication_PutState_Request& Request,
			FConcertReplication_PutState_Response& Response,
			TMap<FGuid, FConcertReplication_ClientChangeData>& ClientChanges
		);
		
		// Leaving
		void HandleLeaveReplicationSessionRequest(const FConcertSessionContext& ConcertSessionContext, const FConcertReplication_LeaveEvent& EventData);
		void OnConnectionChanged(IConcertServerSession& ConcertServerSession, EConcertClientStatus ConcertClientStatus, const FConcertSessionClientInfo& ConcertSessionClientInfo);

		/** Cleans up the client's replication state after leaving. */
		void OnClientLeftReplication(const FGuid& EndpointId);
		/** Calls FConcertServerWorkspace::AddReplicationActivity with the current client state, so it can be restored upon re-joining. */
		void ProduceClientLeftActivity(const FConcertReplicationClient& Client) const;

		/** Generates an activity for a mute request */
		void GenerateMuteActivity(const FGuid& EndpointId, const FConcertReplication_ChangeMuteState_Request& Request) const;
		
		/**
		 * Ticks all clients which causes clients to process pending data and send it to the corresponding endpoints.
		 * 
		 * There is an internal time budget to ensure ticks do not starve the server's other tick tasks.
		 * This is configured in an .ini. TODO: Add config
		 */
		void Tick(IConcertServerSession& InSession, float InDeltaTime);
		
		/** Callback to clients for obtaining an object's frequency settings. */
		FConcertObjectReplicationSettings GetObjectFrequencySettings(const FConcertReplicatedObjectId& Object) const;
	};
}