// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerReplicationManager.h"

#include "AuthorityManager.h"
#include "ConcertLogGlobal.h"
#include "IConcertSession.h"
#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/ConcertReplicationClient.h"
#include "Replication/Messages/ChangeStream.h"
#include "Util/LogUtils.h"
#include "Util/ReplicationCVars.h"
#include "Util/StreamChangeValidation.h"

namespace UE::ConcertSyncServer::Replication
{
	EConcertSessionResponseCode FConcertServerReplicationManager::HandleChangeStreamRequest(
		const FConcertSessionContext& ConcertSessionContext,
		const FConcertReplication_ChangeStream_Request& Request,
		FConcertReplication_ChangeStream_Response& Response)
	{
		LogNetworkMessage(CVarLogStreamRequestsAndResponsesOnServer, Request, [&](){ return GetClientName(*Session, ConcertSessionContext.SourceEndpointId); });
		Response = {};
		
		const FGuid SendingClientId = ConcertSessionContext.SourceEndpointId;
		const TUniquePtr<FConcertReplicationClient>* SendingClient = Clients.Find(SendingClientId);
		if (SendingClient && ValidateStreamChangeRequest(SendingClient->Get()->GetClientEndpointId(), SendingClient->Get()->GetStreamDescriptions(), AuthorityManager, Request, Response))
		{
			ApplyChangeStreamRequest(Request, *SendingClient->Get());
		}
		else
		{
			UE_LOGF(LogConcert, Warning, "Rejecting ChangeStream request from %ls", *SendingClientId.ToString(EGuidFormats::Short));
		}
		
		Response.ErrorCode = EReplicationResponseErrorCode::Handled;
		LogNetworkMessage(CVarLogStreamRequestsAndResponsesOnServer, Response, [&](){ return GetClientName(*Session, ConcertSessionContext.SourceEndpointId); });
		return EConcertSessionResponseCode::Success;
	}

	void FConcertServerReplicationManager::ApplyChangeStreamRequest(const FConcertReplication_ChangeStream_Request& Request, FConcertReplicationClient& Client)
	{
		const FGuid& SendingClientId = Client.GetClientEndpointId();
		const TArray<FConcertReplicationStream>& StreamsBeforeChange = Client.GetStreamDescriptions();

		TArray<FConcertObjectInStreamID> AddedObjects;
		ConcertSyncCore::Replication::ChangeStreamUtils::ForEachAddedObject(Request, StreamsBeforeChange, [&AddedObjects](const FConcertObjectInStreamID& Object)
		{
			AddedObjects.Add(Object);
			return EBreakBehavior::Continue;
		});
		// If the client had authority over any objects that were removed by this request, authority must be cleaned up
		TArray<FConcertObjectInStreamID> RemovedObjects;
		ConcertSyncCore::Replication::ChangeStreamUtils::ForEachRemovedObject(Request, StreamsBeforeChange,
			[this, &SendingClientId, &RemovedObjects](const FConcertObjectInStreamID& RemovedObject)
			{
				AuthorityManager.RemoveAuthority({ RemovedObject, SendingClientId});
				RemovedObjects.Add(RemovedObject);
				return EBreakBehavior::Continue;
			});
			
		// This needs to happen last since all of the above use ChangeStreamUtils::ForEachRemovedObject to determine the diff of changes.
		Client.ApplyValidatedRequest(Request);
			
		ServerObjectCache.OnChangeStreams(SendingClientId, AddedObjects, RemovedObjects);
		MuteManager.PostApplyStreamChange(SendingClientId, AddedObjects, RemovedObjects);
	}
}

