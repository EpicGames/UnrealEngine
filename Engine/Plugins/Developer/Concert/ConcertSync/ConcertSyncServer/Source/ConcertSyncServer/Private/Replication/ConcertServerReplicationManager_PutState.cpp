// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogGlobal.h"
#include "ConcertServerReplicationManager.h"
#include "Algo/AllOf.h"
#include "ConcertServer/Private/ConcertServerSession.h"
#include "Replication/Messages/ChangeClientEvent.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Muting/PredictedStateObjectHierarchy.h"
#include "Replication/Misc/SyncControlUtils.h"
#include "Util/StreamChangeValidation.h"

#if UE_BUILD_DEBUG
#include "JsonObjectConverter.h"
#endif

namespace UE::ConcertSyncServer::Replication
{
	namespace Private
	{
#if UE_BUILD_DEBUG
		static FString RequestToString(const FConcertReplication_PutState_Request& Request)
		{
			FString JsonString;
			FJsonObjectConverter::UStructToJsonObjectString(FConcertReplication_PutState_Request::StaticStruct(), &Request, JsonString, 0, 0);
			return JsonString;
		}
#endif

		static TMap<FGuid, FConcertReplication_ChangeStream_Request> BuildStreamRequests(
			const TMap<FGuid, TUniquePtr<FConcertReplicationClient>>& Clients,
			const FConcertReplication_PutState_Request& Request
			)
		{
			TMap<FGuid, FConcertReplication_ChangeStream_Request> Result;

			for (const TPair<FGuid, FConcertReplicationStreamArray>& Pair : Request.NewStreams)
			{
				const FGuid& EndpointId = Pair.Key;
				const TUniquePtr<FConcertReplicationClient>* Client = Clients.Find(EndpointId);
				if (!Client)
				{
					continue;
				}

				FConcertReplication_ChangeStream_Request& ChangeStream = Result.Add(EndpointId);
				Algo::Transform(Client->Get()->GetStreamDescriptions(), ChangeStream.StreamsToRemove, [](const FConcertReplicationStream& Stream){ return Stream.BaseDescription.Identifier; });
				ChangeStream.StreamsToAdd = Pair.Value.Streams;
			}

			return Result;
		}

		static FConcertReplication_ChangeAuthority_Request BuildAuthorityRequest(
			const FGuid& EndpointId,
			const TArray<FConcertReplicationStream>& Streams,
			const FAuthorityManager& AuthorityManager,
			const TArray<FConcertObjectInStreamID>& ObjectsToOwn
			)
		{
			FConcertReplication_ChangeAuthority_Request AuthorityRequest;
			for (const FConcertReplicationStream& Stream : Streams)
			{
				const FGuid& StreamId = Stream.BaseDescription.Identifier;
				AuthorityManager.EnumerateAuthority(EndpointId, StreamId, [&ObjectsToOwn, &AuthorityRequest, &StreamId](const FSoftObjectPath& ObjectPath)
				{
					const FConcertObjectInStreamID ObjectId{ StreamId, ObjectPath };
					if (!ObjectsToOwn.Contains(ObjectId))
					{
						AuthorityRequest.ReleaseAuthority.Add(ObjectPath).StreamIds.AddUnique(StreamId);
					}
					return EBreakBehavior::Continue;
				});
			}
			for (const FConcertObjectInStreamID& ObjectId : ObjectsToOwn)
			{
				AuthorityRequest.TakeAuthority.FindOrAdd(ObjectId.Object).StreamIds.AddUnique(ObjectId.StreamId);
			}
			return AuthorityRequest;
		}

		static bool ValidateAllClientsKnown(
			const TMap<FGuid, TUniquePtr<FConcertReplicationClient>>& Clients,
			const FConcertReplication_PutState_Request& Request,
			FConcertReplication_PutState_Response& Response
			)
		{
			if (EnumHasAnyFlags(Request.Flags, EConcertReplicationPutStateFlags::SkipDisconnectedClients))
			{
				return true;
			}
			
			bool bSuccess = true;
			
			for (const TPair<FGuid, FConcertReplicationStreamArray>& Pair : Request.NewStreams)
			{
				const FGuid& ClientId = Pair.Key;
				if (!Clients.Contains(ClientId))
				{
					bSuccess = false;
					Response.UnknownEndpoints.Add(ClientId);
				}
			}
			
			for (const TPair<FGuid, FConcertObjectInStreamArray>& Pair : Request.NewAuthorityState)
			{
				const FGuid& ClientId = Pair.Key;
				if (!Clients.Contains(ClientId))
				{
					bSuccess = false;
					Response.UnknownEndpoints.Add(ClientId);
				}
			}

			Response.ResponseCode = bSuccess ? Response.ResponseCode : EConcertReplicationPutStateResponseCode::ClientUnknown;
			return bSuccess;
		}

		static bool ValidateStreamsAreValid(
			const TMap<FGuid, TUniquePtr<FConcertReplicationClient>>& Clients,
			const FAuthorityManager& AuthorityManager,
			const TMap<FGuid, FConcertReplication_ChangeStream_Request>& StreamRequests,
			FConcertReplication_PutState_Response& Response
			)
		{
			const bool bSuccess = Algo::AllOf(StreamRequests, [&Clients, &AuthorityManager](const TPair<FGuid, FConcertReplication_ChangeStream_Request>& Pair)
			{
				const TUniquePtr<FConcertReplicationClient>* Client = Clients.Find(Pair.Key);
				return !Client || ValidateStreamChangeRequest(Client->Get()->GetClientEndpointId(), Client->Get()->GetStreamDescriptions(), AuthorityManager, Pair.Value);
			});

			Response.ResponseCode = bSuccess ? Response.ResponseCode : EConcertReplicationPutStateResponseCode::StreamError;
			return bSuccess;
		}

		/** This checks that Request generates no conflicts with any of the clients that are NOT modified by Request. */
		static bool ValidateNoConflicts(
			const FAuthorityManager& AuthorityManager,
			const FConcertReplication_PutState_Request& Request,
			FConcertReplication_PutState_Response& Response
			)
		{
			bool bSuccess = true;
			
			for (const TPair<FGuid, FConcertObjectInStreamArray>& Pair : Request.NewAuthorityState)
			{
				const FGuid& ClientId = Pair.Key;

				for (const FConcertObjectInStreamID& ObjectId : Pair.Value.Objects)
				{
					const FConcertReplicatedObjectId ReplicatedObjectId { { ObjectId }, ClientId };
					const auto HandleConflict = [&Response, &ClientId, &ReplicatedObjectId](const FGuid& ConflictingClientId, const FGuid& StreamId, const FConcertPropertyChain& Property)
					{
						FConcertAuthorityConflictArray& Conflicts = Response.AuthorityChangeConflicts.FindOrAdd(ClientId);
						Conflicts.FindOrAdd(ReplicatedObjectId).ConflictingObject = { { StreamId, ReplicatedObjectId.Object }, ConflictingClientId };
						return EBreakBehavior::Continue;
					};
					
					const FAuthorityManager::EAuthorityResult AuthorityResult = AuthorityManager.EnumerateAuthorityConflictsWithOverrides(
						ReplicatedObjectId,
						Request.NewStreams,
						// Passing in NewAuthorityState causes us to get a conflict when the request specifies overlapping authority,
						// e.g. if the request says client 1 and client 2 are supposed to replicate the same object properties. 
						Request.NewAuthorityState,
						HandleConflict
						);
					bSuccess &= AuthorityResult == FAuthorityManager::EAuthorityResult::Allowed;
				}
			}

			Response.ResponseCode = bSuccess ? Response.ResponseCode : EConcertReplicationPutStateResponseCode::AuthorityConflict;
			return bSuccess;
		}

		static bool ValidateMuteRequest(
			const TMap<FGuid, TUniquePtr<FConcertReplicationClient>>& Clients,
			const FMuteManager& MuteManager,
			const EConcertSyncSessionFlags SessionFlags,
			const FConcertReplication_PutState_Request& Request,
			FConcertReplication_PutState_Response& Response
			)
		{
			if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldAllowGlobalMuting))
			{
				const bool bIsAllowed = Request.MuteChange.IsEmpty();
				Response.ResponseCode = bIsAllowed ? Response.ResponseCode : EConcertReplicationPutStateResponseCode::FeatureDisabled;
				return bIsAllowed;
			}

			// TODO UE-219829: If EConcertReplicationPutStateFlags::SetStateForDisconnectedClients is set, remove mute changes for unknown objects
			// and put them into the PutState activity, so the mute state can be applied in a RestoreContent request.
			FPredictedStateObjectHierarchy FutureHierarchy;
			FutureHierarchy.AddClients(Request.NewStreams);
			FutureHierarchy.AddClients(Clients, [&Request](const FGuid& ClientId){ return !Request.NewStreams.Contains(ClientId); });
			const bool bIsRequestValid = MuteManager.ValidateRequest(Request.MuteChange, &FutureHierarchy);

			Response.ResponseCode = bIsRequestValid ? Response.ResponseCode : EConcertReplicationPutStateResponseCode::MuteError;
			return bIsRequestValid;
		}
		
		static bool ValidatePutStateRequest(
			const TMap<FGuid, TUniquePtr<FConcertReplicationClient>>& Clients,
			const FAuthorityManager& AuthorityManager,
			const FMuteManager& MuteManager,
			const EConcertSyncSessionFlags SessionFlags,
			const FConcertReplication_PutState_Request& Request,
			const TMap<FGuid, FConcertReplication_ChangeStream_Request>& StreamRequests,
			FConcertReplication_PutState_Response& Response
			)
		{
			return ValidateAllClientsKnown(Clients, Request, Response)
				&& ValidateStreamsAreValid(Clients, AuthorityManager, StreamRequests, Response)
				&& ValidateNoConflicts(AuthorityManager, Request, Response)
				&& ValidateMuteRequest(Clients, MuteManager, SessionFlags, Request, Response);
		}

		static void ApplyPutState_Mute(
			FMuteManager& MuteManager,
			const FGuid& RequestingEndpointId,
			const FConcertReplication_ChangeMuteState_Request& Request,
			FConcertReplication_PutState_Response& Response,
			TMap<FGuid, FConcertReplication_ClientChangeData>& ClientChanges
			)
		{
			const bool bSuccess = MuteManager.ApplyRequestAndEnumerateSyncControl(
			Request,
			[&RequestingEndpointId, &Response, &ClientChanges](const FGuid& EndpointId, const FConcertReplication_ChangeSyncControl& SyncControlChange)
			{
				if (SyncControlChange.IsEmpty())
				{
					return;
				}

				using namespace UE::ConcertSyncCore;
				if (EndpointId == RequestingEndpointId)
				{
					AppendSyncControl(Response.SyncControl, SyncControlChange, EAppendSyncControlFlags::SkipLostControl);
				}
				else
				{
					AppendSyncControl(ClientChanges.FindOrAdd(EndpointId).SyncControlChange, SyncControlChange);
				}
			});
			UE_CLOGF(!bSuccess, LogConcert, Error, "Failed to apply mute request as part of put state request from %ls", *RequestingEndpointId.ToString());
		}
	}
	
	EConcertSessionResponseCode FConcertServerReplicationManager::HandlePutStateRequest(
		const FConcertSessionContext& Context,
		const FConcertReplication_PutState_Request& Request,
		FConcertReplication_PutState_Response& Response
		)
	{
		if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldEnableRemoteEditing))
		{
			Response.ResponseCode = EConcertReplicationPutStateResponseCode::FeatureDisabled;
			return EConcertSessionResponseCode::Success;
		}

		const TMap<FGuid, FConcertReplication_ChangeStream_Request> StreamRequests = Private::BuildStreamRequests(Clients, Request);
		if (Private::ValidatePutStateRequest(Clients, AuthorityManager, MuteManager, SessionFlags, Request, StreamRequests, Response))
		{
			Response.ResponseCode = EConcertReplicationPutStateResponseCode::Success;
			ApplyPutStateRequest(Context.SourceEndpointId, Request, StreamRequests, Response);
		}

		return EConcertSessionResponseCode::Success;
	}

	void FConcertServerReplicationManager::ApplyPutStateRequest(
		const FGuid& RequestingEndpointId,
		const FConcertReplication_PutState_Request& Request,
		const TMap<FGuid, FConcertReplication_ChangeStream_Request> StreamRequests,
		FConcertReplication_PutState_Response& Response
		)
	{
		TMap<FGuid, FConcertReplication_ClientChangeData> ChangedSyncControl;
		const TSet<FConcertObjectInStreamID> RequestingClientSyncControlBefore = [this, &RequestingEndpointId]()
		{
			const TSet<FConcertObjectInStreamID>* SyncControl = SyncControlManager.GetClientControlledObjects(RequestingEndpointId);
			return SyncControl ? *SyncControl : TSet<FConcertObjectInStreamID>{};
		}();
		
		ApplyPutState_Streams(RequestingEndpointId, StreamRequests, ChangedSyncControl);
		ApplyPutState_Authority(RequestingEndpointId, RequestingClientSyncControlBefore, Request, Response, ChangedSyncControl);
		if (EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldAllowGlobalMuting))
		{
			Private::ApplyPutState_Mute(MuteManager, RequestingEndpointId, Request.MuteChange, Response, ChangedSyncControl);
		}

		for (TPair<FGuid, FConcertReplication_ClientChangeData>& SyncControlPair : ChangedSyncControl)
		{
			const FGuid& EndpointId = SyncControlPair.Key;
			// The requesting client receives the sync control via Response (which was written by ApplyChangeClients_StreamPart, etc.)
			if (EndpointId != RequestingEndpointId)
			{
				const FConcertReplication_ChangeClientEvent Event { EConcertReplicationChangeClientReason::PutRequest, MoveTemp(SyncControlPair.Value) };
				Session->SendCustomEvent(Event, EndpointId, EConcertMessageFlags::ReliableOrdered);
			}
		}
	}

	void FConcertServerReplicationManager::ApplyPutState_Streams(
		const FGuid& RequestingEndpointId,
		const TMap<FGuid, FConcertReplication_ChangeStream_Request> StreamRequests,
		TMap<FGuid, FConcertReplication_ClientChangeData>& ClientChanges
	)
	{
		for (const TPair<FGuid, FConcertReplication_ChangeStream_Request>& StreamPair : StreamRequests)
		{
			const FGuid& EndpointId = StreamPair.Key;
			
			const FConcertReplication_ChangeStream_Request& StreamRequest = StreamPair.Value;
			if (EndpointId != RequestingEndpointId)
			{
				ClientChanges.Add(EndpointId).StreamChange = StreamRequest;
			}

			// StreamRequests was created by BuildStreamRequests, which only adds entries for valid clients.
			FConcertReplicationClient& Client = *Clients[EndpointId];
			ApplyChangeStreamRequest(StreamRequest, Client);
		}
	}

	void FConcertServerReplicationManager::ApplyPutState_Authority(
		const FGuid& RequestingEndpointId,
		const TSet<FConcertObjectInStreamID> RequestingClientSyncControlBefore,
		const FConcertReplication_PutState_Request& Request,
		FConcertReplication_PutState_Response& Response,
		TMap<FGuid, FConcertReplication_ClientChangeData>& ClientChanges
	)
	{
		for (const TPair<FGuid, FConcertObjectInStreamArray>& AuthorityPair : Request.NewAuthorityState)
		{
			const FGuid& EndpointId = AuthorityPair.Key;
			
			// EConcertReplicationChangeClientsFlags::SkipDisconnectedClients flag may be set.
			const TUniquePtr<FConcertReplicationClient>* Client = Clients.Find(EndpointId);
			if (!Client)
			{
				continue;
			}

			const TArray<FConcertObjectInStreamID> ObjectsToOwn = AuthorityPair.Value.Objects;
			const FConcertReplication_ChangeAuthority_Request AuthorityRequest = Private::BuildAuthorityRequest(EndpointId, Client->Get()->GetStreamDescriptions(), AuthorityManager, ObjectsToOwn);
			TMap<FSoftObjectPath, FConcertStreamArray> RejectedObjects;
			if (EndpointId == RequestingEndpointId)
			{
				FConcertReplication_ChangeSyncControl& RequesterSyncControlChange = Response.SyncControl;
				AuthorityManager.ApplyChangeAuthorityRequest(EndpointId, AuthorityRequest, RejectedObjects, Response.SyncControl);
				// Only return actual changes of sync control to the requesting client.
				for (auto It = RequesterSyncControlChange.NewControlStates.CreateIterator(); It; ++It)
				{
					if (RequestingClientSyncControlBefore.Contains(It->Key))
					{
						It.RemoveCurrent();
					}
				}
			}
			else
			{
				ClientChanges.FindOrAdd(EndpointId).AuthorityChange = AuthorityRequest;
				FConcertReplication_ChangeSyncControl& RemoteSyncControlChange = ClientChanges.FindOrAdd(EndpointId).SyncControlChange;
				AuthorityManager.ApplyChangeAuthorityRequest(EndpointId, AuthorityRequest, RejectedObjects, RemoteSyncControlChange);
			}

#if UE_BUILD_DEBUG
			ensure(RejectedObjects.IsEmpty());
			UE_CLOGF(!RejectedObjects.IsEmpty(), LogConcert, Warning, "Authority portion for client %ls was not validated correctly for request %ls.", *EndpointId.ToString(), *Private::RequestToString(Request));
#endif
		}
	}
}
