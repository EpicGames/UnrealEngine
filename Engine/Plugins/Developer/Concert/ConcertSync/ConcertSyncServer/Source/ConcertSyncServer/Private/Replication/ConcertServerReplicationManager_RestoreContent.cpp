// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertLogGlobal.h"

#include "ConcertMessageData.h"
#include "ConcertServerReplicationManager.h"
#include "ConcertServer/Private/ConcertServerSession.h"
#include "Replication/IReplicationWorkspace.h"
#include "Replication/MuteUtils.h"
#include "Replication/Messages/ChangeAuthority.h"
#include "Replication/Messages/ChangeStream.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Replication/Misc/ReplicationStreamUtils.h"
#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"

namespace UE::ConcertSyncServer::Replication
{
	namespace RestoreContentPrivate
	{
		/** Checks whether the request should fail according to EConcertReplicationRestoreContentFlags::ValidateUniqueClient. */
		static bool HasNameConflict(
			const FConcertSessionClientInfo& SenderInfo,
			const FConcertReplication_RestoreContent_Request& Request,
			const IConcertServerSession& Session
			)
		{
			if (!EnumHasAnyFlags(Request.Flags, EConcertReplicationRestoreContentFlags::ValidateUniqueClient))
			{
				return false;
			}
			
			for (const FGuid& SessionId : Session.GetSessionClientEndpointIds())
			{
				FConcertSessionClientInfo OtherInfo;
				const bool bFoundClient = Session.FindSessionClient(SessionId, OtherInfo);

				if (ensure(bFoundClient)
					&& OtherInfo.ClientEndpointId != SenderInfo.ClientEndpointId
					&& ConcertSyncCore::Replication::AreLogicallySameClients(SenderInfo.ClientInfo, OtherInfo.ClientInfo))
				{
					return true;
				}
			}
			return false;
		}

		/** Checks whether the request should fail according to EConcertReplicationAuthorityRestoreMode::AllOrNothing. */
		static bool HasAuthorityConflict(
			const FConcertReplication_RestoreContent_Request& Request,
			const FConcertSyncReplicationPayload_LeaveReplication& ClientState,
			const FAuthorityManager& AuthorityManager
			)
		{
			if (Request.AuthorityRestorationMode != EConcertReplicationAuthorityRestoreMode::AllOrNothing)
			{
				// Request wants to fail irrelevant on whether EConcertReplicationRestoreContentFlags::RestoreAuthority is set
				return false;
			}

			for (const FConcertObjectInStreamID& PreviouslyOwnedObject : ClientState.OwnedObjects)
			{
				const FConcertReplicationStream* Stream = ClientState.Streams.FindByPredicate([StreamId = PreviouslyOwnedObject.StreamId](const FConcertReplicationStream& Stream)
				{
					return Stream.BaseDescription.Identifier == StreamId;
				});

				const FConcertReplicatedObjectInfo* ObjectInfo = Stream ? Stream->BaseDescription.ReplicationMap.ReplicatedObjects.Find(PreviouslyOwnedObject.Object) : nullptr;
				if (!ensure(ObjectInfo))
				{
					UE_LOGF(LogConcert, Error, "Saved activity state violates invariant. Object %ls is owned but does not appear in stream", *PreviouslyOwnedObject.ToString());
					continue;
				}

				const FConcertReplicatedObjectId ObjectId{ { PreviouslyOwnedObject }, Stream->BaseDescription.Identifier };
				const bool bHasConflict = AuthorityManager.EnumerateAuthorityConflicts(ObjectId, &ObjectInfo->PropertySelection) == FAuthorityManager::EAuthorityResult::Conflict;
				if (bHasConflict)
				{
					return true;
				}
			}

			return false;
		}

		/** Fills OutChangedSyncControl with all objects the Client currently has sync control over. */
		static void BuildRemovedSyncControl(const FConcertReplicationClient& Client, const FSyncControlManager& SyncControlManager, FConcertReplication_ChangeSyncControl& OutChangedSyncControl)
		{
			for (const FConcertReplicationStream& ExistingStream : Client.GetStreamDescriptions())
			{
				for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Object : ExistingStream.BaseDescription.ReplicationMap.ReplicatedObjects)
				{
					const FConcertObjectInStreamID ObjectId { ExistingStream.BaseDescription.Identifier, Object.Key };
					const FConcertReplicatedObjectId ReplicatedObjectId { { ObjectId }, Client.GetClientEndpointId() };
					if (SyncControlManager.HasSyncControl(ReplicatedObjectId))
					{
						OutChangedSyncControl.NewControlStates.Add(ObjectId, false);
					}
				}
			}
		}

		/** Adds ObjectId either to StreamsToAdd or ObjectsToPut depending on whether ClientStreams already contains it. */
		static void AddObjectToRequest(
			FConcertReplication_ChangeStream_Request& ChangeRequest,
			const TArray<FConcertReplicationStream>& ClientStreams,
			const FConcertObjectInStreamID& ObjectId,
			const FConcertReplicatedObjectInfo& ObjectInfo,
			const FConcertReplicationStream& RestoredStream
			)
		{
			const FConcertObjectReplicationSettings* ObjectFrequencyOverride = RestoredStream.BaseDescription.FrequencySettings.ObjectOverrides.Find(ObjectId.Object);
			if (const FConcertReplicationStream* ExistingStream = ConcertSyncCore::FindStream(ClientStreams, ObjectId.StreamId))
			{
				FConcertReplication_ChangeStream_PutObject& PutObject = ChangeRequest.ObjectsToPut.Add(ObjectId);
				PutObject.Properties = ObjectInfo.PropertySelection;
				PutObject.ClassPath = ObjectInfo.ClassPath;
				if (const FConcertReplicatedObjectInfo* ExistingObjectInfo = ConcertSyncCore::FindObjectInfo(*ExistingStream, ObjectId.Object))
				{
					PutObject.Properties.ReplicatedProperties.Append(ExistingObjectInfo->PropertySelection.ReplicatedProperties);
				}
				
				if (ObjectFrequencyOverride)
				{
					ChangeRequest.FrequencyChanges.FindOrAdd(ObjectId.StreamId).OverridesToAdd.Add(ObjectId.Object) = *ObjectFrequencyOverride;
				}
				return;
			}
				
			FConcertReplicationStream* AddedStream = ConcertSyncCore::FindStreamEditable(ChangeRequest.StreamsToAdd, ObjectId.StreamId);
			if (!AddedStream)
			{
				const int32 NewStreamIndex = ChangeRequest.StreamsToAdd.Emplace();
				AddedStream = &ChangeRequest.StreamsToAdd[NewStreamIndex];
				AddedStream->BaseDescription.Identifier = ObjectId.StreamId;
				AddedStream->BaseDescription.FrequencySettings.Defaults = RestoredStream.BaseDescription.FrequencySettings.Defaults;
			}
				
			AddedStream->BaseDescription.ReplicationMap.ReplicatedObjects.Add(ObjectId.Object, ObjectInfo);
			if (ObjectFrequencyOverride)
			{
				AddedStream->BaseDescription.FrequencySettings.ObjectOverrides.Add(ObjectId.Object, *ObjectFrequencyOverride);
			}
		}

		/** Lists every objects that EndpointId can take authority over. */
		template<typename TLambda> requires std::is_invocable_v<TLambda, const FConcertObjectInStreamID&>
		static void ForEachUnownedObject(const FGuid& EndpointId, const TArray<FConcertObjectInStreamID>& Objects, const FAuthorityManager& AuthorityManager, TLambda&& Callback)
		{
			for (const FConcertObjectInStreamID& ObjectId : Objects)
			{
				const FConcertReplicatedObjectId ReplicatedObjectId { { ObjectId }, EndpointId };
				const bool bIsAllowed = AuthorityManager.EnumerateAuthorityConflicts(ReplicatedObjectId) == FAuthorityManager::EAuthorityResult::Allowed;
				if (bIsAllowed)
				{
					Callback(ObjectId);
				}
			}
		}

		static void FillResponseWithClientState(
			const FConcertReplication_RestoreContent_Request& Request,
			FConcertReplication_RestoreContent_Response& Response,
			const FConcertReplicationClient& Client,
			const FAuthorityManager& AuthorityManager
			)
		{
			if (!EnumHasAnyFlags(Request.Flags, EConcertReplicationRestoreContentFlags::SendNewState))
			{
				return;
			}
			
			for (const FConcertReplicationStream& Stream : Client.GetStreamDescriptions())
			{
				const FGuid& StreamId = Stream.BaseDescription.Identifier;
				Response.ClientInfo.Streams.Add(Stream.BaseDescription);
				AuthorityManager.EnumerateAuthority(Client.GetClientEndpointId(), Stream.BaseDescription.Identifier,
					[&Response, &StreamId](const FSoftObjectPath& OwnedObject)
					{
						TArray<FConcertAuthorityClientInfo>& Authorities = Response.ClientInfo.Authority;
						const int32 Index = Authorities.IndexOfByPredicate([&StreamId](const FConcertAuthorityClientInfo& Info)
						{
							return Info.StreamId == StreamId;
						});
						FConcertAuthorityClientInfo& ClientInfo = Authorities.IsValidIndex(Index)
							? Authorities[Index]
							: [&StreamId, &Authorities]() -> FConcertAuthorityClientInfo&
							{
								const int32 NewIndex = Authorities.Add(FConcertAuthorityClientInfo{ StreamId });
								return Authorities[NewIndex];
							}();
						
						ClientInfo.AuthoredObjects.AddUnique(OwnedObject);
						return EBreakBehavior::Continue;
					});
			}
		}
	}
	
	EConcertSessionResponseCode FConcertServerReplicationManager::HandleRestoreContentRequest(
		const FConcertSessionContext& ConcertSessionContext,
		const FConcertReplication_RestoreContent_Request& Request,
		FConcertReplication_RestoreContent_Response& Response)
	{
		const auto[ErrorCode, LeaveReplicationPayload] = ValidateRestoreContentRequest(ConcertSessionContext.SourceEndpointId, Request);
		if (LeaveReplicationPayload)
		{
			ApplyRestoreContentRequest(ConcertSessionContext.SourceEndpointId, Request, *LeaveReplicationPayload, Response.SyncControl);
			RestoreContentPrivate::FillResponseWithClientState(Request, Response, *Clients[ConcertSessionContext.SourceEndpointId], AuthorityManager);
		}
		
		Response.ErrorCode = ErrorCode;
		return EConcertSessionResponseCode::Success;
	}

	TTuple<EConcertReplicationRestoreErrorCode, TOptional<FConcertSyncReplicationPayload_LeaveReplication>> FConcertServerReplicationManager::ValidateRestoreContentRequest(
		const FGuid& RequestingEndpointId,
		const FConcertReplication_RestoreContent_Request& Request
		) const
	{
		if (!Clients.Contains(RequestingEndpointId))
		{
			return MakeTuple(EConcertReplicationRestoreErrorCode::Invalid, TOptional<FConcertSyncReplicationPayload_LeaveReplication>{});
		}
		
		if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldEnableReplicationActivities))
		{
			return MakeTuple(EConcertReplicationRestoreErrorCode::NotSupported, TOptional<FConcertSyncReplicationPayload_LeaveReplication>{});
		}
		
		FConcertSessionClientInfo ClientInfo;
		const bool bFoundClient = Session->FindSessionClient(RequestingEndpointId, ClientInfo);
		check(bFoundClient);

		// Does the request want to be skipped if there is another client?
		if (RestoreContentPrivate::HasNameConflict(ClientInfo, Request, *Session))
		{
			return MakeTuple(EConcertReplicationRestoreErrorCode::NameConflict, TOptional<FConcertSyncReplicationPayload_LeaveReplication>{});
		}

		// Is there even anything to restore to?
		FConcertSyncReplicationPayload_LeaveReplication OldClientState;
		const bool bOverrideActivity = Request.ActivityId.IsSet();
		const bool bFound = bOverrideActivity
			? ServerWorkspace.GetLeaveReplicationEventById(*Request.ActivityId, OldClientState)
			: ServerWorkspace.GetLastLeaveReplicationActivityByClient(ClientInfo, OldClientState);
		if (!bFound)
		{
			return MakeTuple(
				bOverrideActivity ? EConcertReplicationRestoreErrorCode::NoSuchActivity : EConcertReplicationRestoreErrorCode::Success,
				TOptional<FConcertSyncReplicationPayload_LeaveReplication>{}
				);
		}

		// Does the request want to fail if another client already has authority over any of the objects?
		if (RestoreContentPrivate::HasAuthorityConflict(Request, OldClientState, AuthorityManager))
		{
			return MakeTuple(EConcertReplicationRestoreErrorCode::AuthorityConflict, TOptional<FConcertSyncReplicationPayload_LeaveReplication>{});
		}

		return MakeTuple(EConcertReplicationRestoreErrorCode::Success, TOptional(OldClientState));
	}

	void FConcertServerReplicationManager::ApplyRestoreContentRequest(
		const FGuid& RequestingEndpointId,
		const FConcertReplication_RestoreContent_Request& Request,
		const FConcertSyncReplicationPayload_LeaveReplication& DataToApply, 
		FConcertReplication_ChangeSyncControl& ChangedSyncControl
		)
	{
		FConcertReplicationClient& Client = *Clients[RequestingEndpointId];
		
		// After this first step, SyncControl hold the objects that the client had sync control over before but that are removed by this request.
		RestoreStreamContent(Request, DataToApply, Client, ChangedSyncControl);
		
		if (EnumHasAnyFlags(Request.Flags, EConcertReplicationRestoreContentFlags::RestoreAuthority))
		{
			// This adds the objects the client receives sync control for
			RestoreAuthority(DataToApply, Client, ChangedSyncControl);
		}

		if (EnumHasAnyFlags(Request.Flags, EConcertReplicationRestoreContentFlags::RestoreMute))
		{
			RestoreMuteState(Client, ChangedSyncControl);
		}
	}
	
	void FConcertServerReplicationManager::RestoreStreamContent(
		const FConcertReplication_RestoreContent_Request& Request,
		const FConcertSyncReplicationPayload_LeaveReplication& DataToApply,
		FConcertReplicationClient& Client,
		FConcertReplication_ChangeSyncControl& OutChangedSyncControl
		)
	{
		const bool bClearedContent = !EnumHasAnyFlags(Request.Flags, EConcertReplicationRestoreContentFlags::RestoreOnTop); 
		if (bClearedContent)
		{
			// We'll act as if all objects will remove sync control. In RestoreAuthority, we'll fix it back up.
			RestoreContentPrivate::BuildRemovedSyncControl(Client, SyncControlManager, OutChangedSyncControl);
			
			FConcertReplication_ChangeStream_Request ClearRequest;
			Algo::Transform(Client.GetStreamDescriptions(), ClearRequest.StreamsToRemove, [](const FConcertReplicationStream& Stream){ return Stream.BaseDescription.Identifier; });
			ApplyChangeStreamRequest(ClearRequest, Client);
		}

		FConcertReplication_ChangeStream_Request ChangeRequest;
		for (const FConcertReplicationStream& Stream : DataToApply.Streams)
		{
			const FGuid& StreamId = Stream.BaseDescription.Identifier;
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				const FConcertReplicatedObjectId ObjectId{ { StreamId, Pair.Key }, Client.GetClientEndpointId() };

				// For the IncludeAlreadyOwnedObjectPropertiesInStream case, we'll add to the stream regardless of other clients' authority states.
				// For the AllOrNothing case, ValidateRestoreContentRequest has already validated there won't be any conflicts.
				const bool bShouldSkip = Request.AuthorityRestorationMode == EConcertReplicationAuthorityRestoreMode::ExcludeAlreadyOwnedObjectPropertiesFromStream
					&& AuthorityManager.EnumerateAuthorityConflicts(ObjectId, &Pair.Value.PropertySelection) == FAuthorityManager::EAuthorityResult::Conflict;
				if (!bShouldSkip)
				{
					RestoreContentPrivate::AddObjectToRequest(ChangeRequest, Client.GetStreamDescriptions(), ObjectId, Pair.Value, Stream);
				}
			}

			// The stream's frequency settings also need to be restored
			const FConcertReplicationStream* PreExistingStream = ConcertSyncCore::FindStream(Client.GetStreamDescriptions(), StreamId);
			const FConcertObjectReplicationSettings& RestoreFrequencyDefaults = Stream.BaseDescription.FrequencySettings.Defaults;
			if (PreExistingStream && PreExistingStream->BaseDescription.FrequencySettings.Defaults != RestoreFrequencyDefaults)
			{
				ChangeRequest.FrequencyChanges.FindOrAdd(StreamId).NewDefaults = RestoreFrequencyDefaults; 
			}
		}
		ApplyChangeStreamRequest(ChangeRequest, Client);
	}

	void FConcertServerReplicationManager::RestoreAuthority(
		const FConcertSyncReplicationPayload_LeaveReplication& DataToApply,
		const FConcertReplicationClient& Client,
		FConcertReplication_ChangeSyncControl& OutChangedSyncControl
		)
	{
		// Assign every object to Client that would not cause any conflict
		FConcertReplication_ChangeAuthority_Request Request;
		RestoreContentPrivate::ForEachUnownedObject(Client.GetClientEndpointId(), DataToApply.OwnedObjects, AuthorityManager,
			[&Request](const FConcertObjectInStreamID& AllowedObject)
			{
				Request.TakeAuthority.FindOrAdd(AllowedObject.Object).StreamIds.AddUnique(AllowedObject.StreamId);
			});

		// So far OutChangedSyncControl was filled by RestoreStreamContent: it contains only objects losing sync control.
		// Some of those objects may now regain sync control and will be overriden.
		TMap<FSoftObjectPath, FConcertStreamArray> RejectedObjects;
		AuthorityManager.ApplyChangeAuthorityRequest(Client.GetClientEndpointId(), Request, RejectedObjects, OutChangedSyncControl);
		check(RejectedObjects.IsEmpty());
	}

	void FConcertServerReplicationManager::RestoreMuteState(const FConcertReplicationClient& Client, FConcertReplication_ChangeSyncControl& OutChangedSyncControl)
	{
		class FMuteStateGroundTruth : public ConcertSyncCore::Replication::MuteUtils::IMuteStateGroundTruth
		{
			const FMuteManager& MuteManager;
			const ConcertSyncCore::FReplicatedObjectHierarchyCache& ServerObjectCache;
		public:
			
			FMuteStateGroundTruth(
				const FMuteManager& MuteManager UE_LIFETIMEBOUND,
				const ConcertSyncCore::FReplicatedObjectHierarchyCache& ServerObjectCache UE_LIFETIMEBOUND
				)
				: MuteManager(MuteManager)
				, ServerObjectCache(ServerObjectCache)
			{}

			virtual ConcertSyncCore::Replication::MuteUtils::EMuteState GetMuteState(const FSoftObjectPath& Object) const override
			{
				using namespace ConcertSyncCore::Replication::MuteUtils;
				
				const TOptional<FMuteManager::EMuteState> MuteState = MuteManager.GetMuteState(Object);
				if (!MuteState)
				{
					return EMuteState::None;
				}

				switch (*MuteState)
				{
				case FMuteManager::EMuteState::ExplicitlyMuted: return EMuteState::ExplicitlyMuted;
				case FMuteManager::EMuteState::ExplicitlyUnmuted: return EMuteState::ExplicitlyUnmuted;
				case FMuteManager::EMuteState::ImplicitlyMuted: return EMuteState::ImplicitlyMuted;
				case FMuteManager::EMuteState::ImplicitlyUnmuted: return EMuteState::ImplicitlyUnmuted;
				default: checkNoEntry(); return EMuteState::None;
				}
			}
			
			virtual TOptional<FConcertReplication_ObjectMuteSetting> GetExplicitSetting(const FSoftObjectPath& Object) const override
			{
				return MuteManager.GetExplicitMuteSetting(Object);
			}
			
			virtual bool IsObjectKnown(const FSoftObjectPath& Object) const override
			{
				return ServerObjectCache.IsInHierarchy(Object).IsSet();
			}
		} GroundTruth(MuteManager, ServerObjectCache);

		// We'll effectively replay all mute actions that have occured so far by combining them into Request.
		FConcertReplication_ChangeMuteState_Request AggregatedRequest;
		ServerWorkspace.EnumerateMuteActivities([&GroundTruth, &AggregatedRequest](const FConcertSyncReplicationActivity& Activity)
		{
			// TODO UE-219951: We need to handle correctly replaying after PutState requests, as well! For that, we must first produce PutState activities, however (UE-219824)! 
			FConcertSyncReplicationPayload_Mute MuteData;
			if (!ensure(Activity.EventData.ActivityType == EConcertSyncReplicationActivityType::Mute)
				|| !Activity.EventData.GetPayload(MuteData))
			{
				return EBreakBehavior::Continue;
			}

			// This skips changes that are already in effect or that are invalid to do (e.g. due to unknown object)
			CombineMuteRequests(AggregatedRequest, MuteData.Request, GroundTruth);
			return EBreakBehavior::Continue;
		});

		if (!AggregatedRequest.IsEmpty())
		{
			ApplyRestoringMuteRequest(Client, AggregatedRequest, OutChangedSyncControl);
		}
	}
	void FConcertServerReplicationManager::ApplyRestoringMuteRequest(
		const FConcertReplicationClient& Client,
		const FConcertReplication_ChangeMuteState_Request& AggregatedRequest,
		FConcertReplication_ChangeSyncControl& OutChangedSyncControl
		)
	{
		const FGuid& ClientId = Client.GetClientEndpointId();
		const FConcertReplication_ChangeSyncControl GainedSyncControl = MuteManager.ApplyManualRequest(ClientId, AggregatedRequest);

		// ApplyManualRequest may have removed sync control but GainedSyncControl does not contain the removed objects...
		for (TPair<FConcertObjectInStreamID, bool>& ResultControl : OutChangedSyncControl.NewControlStates)
		{
			const FConcertReplicatedObjectId ReplicatedObjectId { { ResultControl.Key }, ClientId };
			// ... so simply go through everything and check whether it lost control
			if (!SyncControlManager.HasSyncControl(ReplicatedObjectId))
			{
				ResultControl.Value = false;
			}
		}

		// And all the things that have now gained the control will be put into OutChangedSyncControl afterwards.
		for (const TPair<FConcertObjectInStreamID, bool>& GainedControl : GainedSyncControl.NewControlStates)
		{
			OutChangedSyncControl.NewControlStates.Add(GainedControl.Key, GainedControl.Value);
		}
	}
}
