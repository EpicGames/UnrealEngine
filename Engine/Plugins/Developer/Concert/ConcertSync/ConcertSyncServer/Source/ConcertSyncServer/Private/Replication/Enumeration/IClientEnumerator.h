// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"
#include "Templates/Function.h"

struct FConcertReplicationStream;
struct FGuid;

namespace UE::ConcertSyncServer::Replication
{
	/** Iterates through all clients have registered to send any data. */
	class IClientEnumerator
	{
	public:

		/** Iterates through all clients have registered to send any data. */
		virtual void ForEachReplicationClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const = 0;
		
		virtual ~IClientEnumerator() = default;
	};
}