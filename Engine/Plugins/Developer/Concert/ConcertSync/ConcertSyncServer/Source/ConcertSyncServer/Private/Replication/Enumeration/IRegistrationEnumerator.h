// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IClientEnumerator.h"
#include "IStreamEnumerator.h"

namespace UE::ConcertSyncServer::Replication
{
	class IRegistrationEnumerator : public IStreamEnumerator, public IClientEnumerator
	{};
}