// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"
#include "Templates/Function.h"

struct FConcertReplicationStream;
struct FGuid;

namespace UE::ConcertSyncServer::Replication
{
	/** Provides a way to extract all streams registered to a given client. */
	class IStreamEnumerator
	{
	public:

		/** Provides a way to extract all streams registered to a given client. */
		virtual void ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback) const = 0;

		virtual ~IStreamEnumerator() = default;
	};
}