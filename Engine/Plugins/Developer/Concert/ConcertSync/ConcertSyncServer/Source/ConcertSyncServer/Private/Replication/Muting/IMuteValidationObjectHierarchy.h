// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"

struct FGuid;
struct FSoftObjectPath;

namespace UE::ConcertSyncServer::Replication
{
	/**
	 * Interface required by FMuteManager to validate a FConcertReplication_ChangeMuteState_Request.
	 * 
	 * This encapsulates the server's knowledge of replicated objects.
	 * It is useful if you have a complicated series of requests that will mutate the server state if applied, but you want to first validate that
	 * a mute request is valid to apply on that state. In that case, you can implement this to return the future server state.
	 */
	class IMuteValidationObjectHierarchy
	{
	public:

		/**
		 * Checks whether the object is registered by any client (except for this in IgnoredClients).
		 *
		 * This function ignores implicit knowledge of the hierarchy.
		 * For example if you register ONLY /Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0, then
		 * - IsObjectReferencedDirectly(/Game/Maps.Map:PersistentLevel.Cube.StaticMeshComponent0) == true,
		 * - IsObjectReferencedDirectly(/Game/Maps.Map:PersistentLevel.Cube) == false
		 * but e.g. TraverseTopToBottom would list both paths.
		 *
		 * @see FReplicatedObjectHierarchyCache::IsObjectReferencedDirectly
		 */
		virtual bool IsObjectReferencedDirectly(const FSoftObjectPath& ObjectPath, TConstArrayView<FGuid> IgnoredClients = {}) const = 0;

		/**
		 * @return Whether Object has any subobjects in the hierarchy.
		 * @see FObjectPathHierarchy::HasChildren
		 */
		virtual bool HasChildren(const FSoftObjectPath& Object) const = 0;

		virtual ~IMuteValidationObjectHierarchy() = default;
	};
}