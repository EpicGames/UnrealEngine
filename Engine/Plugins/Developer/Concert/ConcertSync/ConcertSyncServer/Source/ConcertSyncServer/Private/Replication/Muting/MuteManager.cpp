// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuteManager.h"

#include "ConcertLogGlobal.h"
#include "IConcertSession.h"
#include "Misc/ObjectPathOuterIterator.h"
#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/Messages/Muting.h"
#include "Replication/Misc/ReplicatedObjectHierarchyCache.h"
#include "Replication/Muting/ObjectHierarchyAdapter.h"
#include "Replication/Util/LogUtils.h"

#include "HAL/IConsoleManager.h"
#include "Misc/ScopeExit.h"

namespace UE::ConcertSyncServer::Replication
{
	TAutoConsoleVariable<bool> CVarLogMuteRequestsAndResponsesOnServer(
		TEXT("Concert.Replication.LogMuteRequestsAndResponsesOnServer"),
		false,
		TEXT("Whether to log changes to the mute state.")
		);
	
	FMuteManager::FMuteManager(
		IConcertSession& InSession,
		const ConcertSyncCore::FReplicatedObjectHierarchyCache& ServerObjectCache,
		EConcertSyncSessionFlags InSessionFlags
		)
		: Session(InSession)
		, ServerObjectCache(ServerObjectCache)
		, SessionFlags(InSessionFlags)
	{
		Session.RegisterCustomRequestHandler<FConcertReplication_ChangeMuteState_Request, FConcertReplication_ChangeMuteState_Response>(this, &FMuteManager::HandleChangeMuteStateRequest);
		Session.RegisterCustomRequestHandler<FConcertReplication_QueryMuteState_Request, FConcertReplication_QueryMuteState_Response>(this, &FMuteManager::HandleQueryMuteStateRequest);
	}

	FMuteManager::~FMuteManager()
	{
		Session.UnregisterCustomRequestHandler<FConcertReplication_ChangeMuteState_Request>();
		Session.UnregisterCustomRequestHandler<FConcertReplication_QueryMuteState_Request>();
	}

	bool FMuteManager::IsMuted(const FSoftObjectPath& Object) const
	{
		const FMuteData* ObjectMuteState = MuteState.Find(Object);
		return ObjectMuteState
			&& (ObjectMuteState->State == EMuteState::ExplicitlyMuted || ObjectMuteState->State == EMuteState::ImplicitlyMuted);
	}

	TOptional<FMuteManager::EMuteState> FMuteManager::GetMuteState(const FSoftObjectPath& Object) const
	{
		const FMuteData* ObjectMuteState = MuteState.Find(Object);
		return ObjectMuteState
			? ObjectMuteState->State
			: TOptional<EMuteState>{};
	}

	TOptional<FConcertReplication_ObjectMuteSetting> FMuteManager::GetExplicitMuteSetting(const FSoftObjectPath& Object) const
	{
		const FMuteData* ObjectMuteState = MuteState.Find(Object);
		return ObjectMuteState
			? ObjectMuteState->MuteSetting
			: TOptional<FConcertReplication_ObjectMuteSetting>{};
	}

	bool FMuteManager::ValidateRequest(
		const FConcertReplication_ChangeMuteState_Request& Request,
		const IMuteValidationObjectHierarchy* OverrideServerObjectCache,
		TFunctionRef<void(const FSoftObjectPath& ObjectPath)> OnRejection
		) const
	{
		const bool bFeatureSupported = EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldAllowGlobalMuting);
		if (!ensureMsgf(bFeatureSupported, TEXT("Don't call this if the feature is not enabled!")))
		{
			return false;
		}

		return OverrideServerObjectCache
			? ValidateRequestInternal(Request, *OverrideServerObjectCache, OnRejection)
			: ValidateRequestInternal(Request, FObjectHierarchyAdapter(ServerObjectCache), OnRejection);
	}

	bool FMuteManager::ApplyRequestAndEnumerateSyncControl(
		const FConcertReplication_ChangeMuteState_Request& Request,
		const FOnSyncControlChange& OnSyncControlChange
		)
	{
		const bool bIsValid = ValidateRequest(Request);
		if (ensureAlwaysMsgf(bIsValid, TEXT("This is a double-check and it failed. The caller should have validate the request beforehand.")))
		{
			ApplyRequestInternal(Request);

			check(OnRefreshSyncControlButSkipSendingToClientsDelegate.IsBound());
			OnRefreshSyncControlButSkipSendingToClientsDelegate.Execute(OnSyncControlChange);
		}
		
		return bIsValid;
	}

	FConcertReplication_ChangeSyncControl FMuteManager::ApplyManualRequest(const FGuid& EndpointId, const FConcertReplication_ChangeMuteState_Request& Request)
	{
		ApplyRequestInternal(Request);
		return OnRefreshSyncControlAndSendToAllClientsExceptDelegate.Execute(EndpointId);
	}

	void FMuteManager::PostApplyStreamChange(const FGuid& ClientId, TConstArrayView<FConcertObjectInStreamID> AddedObjects, TConstArrayView<FConcertObjectInStreamID> RemovedObjects)
	{
		// If Request adds new subobjects that should be muted implicitly by existing rules, update MuteState.
		TrackAddedSubobjectsForImplicitMuting(AddedObjects);
		// Objects that are no longer referenced by any clients must be removed (to avoid leaks)
		UnmuteObjectsIfUnreferenced(RemovedObjects);
		
		// This updates sync control, which may remove sync control from other clients.
		check(OnMuteStateChangedDelegate.IsBound());
		OnMuteStateChangedDelegate.Execute(ClientId);
	}

	void FMuteManager::OnPostClientLeft(const TArray<FConcertReplicationStream>& ClientStreams)
	{
		// We'll treat this case as a FConcertReplication_ChangeStream_Request that removes all objects 
		TSet<FSoftObjectPath> VisitedObjects;
		for (const FConcertReplicationStream& Stream : ClientStreams)
		{
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ObjectPair : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				const FSoftObjectPath ObjectPath = ObjectPair.Key; 
				if (!VisitedObjects.Contains(ObjectPath))
				{
					VisitedObjects.Add(ObjectPath);
					UnmuteObjectIfUnreferenced(ObjectPath);
				}
			}
		}
	}

	void FMuteManager::TrackAddedSubobjectsForImplicitMuting(TConstArrayView<FConcertObjectInStreamID> AddedObjects)
	{
		for (const FConcertObjectInStreamID& Object : AddedObjects)
		{
			const FSoftObjectPath ObjectPath = Object.Object;
			if (MuteState.Contains(ObjectPath))
			{
				continue;
			}

			const auto[_, ParentMuteData] = FindAffectingParentState(ObjectPath);
			if (ParentMuteData && ParentMuteData->State == EMuteState::ExplicitlyUnmuted)
			{
				MuteState.Add(ObjectPath).State = EMuteState::ImplicitlyMuted;
			}
			else if (ParentMuteData && ParentMuteData->State == EMuteState::ExplicitlyMuted)
			{
				MuteState.Add(ObjectPath).State = EMuteState::ImplicitlyUnmuted;
			}
		}
	}

	void FMuteManager::UnmuteObjectsIfUnreferenced(TConstArrayView<FConcertObjectInStreamID> RemovedObjects)
	{
		// We'll avoid processing any object again that we've already removed... not doing so would unnecessarily iterate the hierarchy.
		TSet<FSoftObjectPath> ProcessedObjects;
		
		for (const FConcertObjectInStreamID& RemovedObject : RemovedObjects)
		{
			const FSoftObjectPath& RemovedObjectPath = RemovedObject.Object;
			if (!ProcessedObjects.Contains(RemovedObjectPath))
			{
				UnmuteObjectIfUnreferenced(RemovedObjectPath, [&ProcessedObjects](const FSoftObjectPath& ObjectPath)
				{
					ProcessedObjects.Add(ObjectPath);
				});
			}
		}
	}

	template<CObjectProcessable TCallback>
	void FMuteManager::UnmuteObjectIfUnreferenced(const FSoftObjectPath& RemovedObject, TCallback&& OnRemoved)
	{
		// If RemovedObject is referenced or has known subobjects, the mute state should tick around.
		const bool bHasChildrenOrIsReferenced = ServerObjectCache.HasChildren(RemovedObject);
		if (bHasChildrenOrIsReferenced)
		{
			return;
		}
		
		const auto [AffectingParent, ParentMuteData] = FindAffectingParentState(RemovedObject);
		// RemovedObject is not referenced by any streams, so it has no business being muted.
		RemoveMuteState(RemovedObject, OnRemoved);

		// There may be a parent object, which also no longer has known children ... so remove that as well.
		if (ParentMuteData)
		{
			UnmuteObjectIfUnreferenced(AffectingParent, OnRemoved);
		}
	}

	EConcertSessionResponseCode FMuteManager::HandleQueryMuteStateRequest(
		const FConcertSessionContext& Context,
		const FConcertReplication_QueryMuteState_Request& Request,
		FConcertReplication_QueryMuteState_Response& Response
		)
	{
		if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldAllowGlobalMuting))
		{
			return EConcertSessionResponseCode::Failed;
		}

		Response.ErrorCode = EReplicationResponseErrorCode::Handled;
		const auto AddObjectToResponse = [this, &Response](const FSoftObjectPath& ObjectPath, const FMuteData& ObjectMuteData)
		{
			switch (ObjectMuteData.State)
			{
			case EMuteState::ExplicitlyMuted:
				Response.ExplicitlyMutedObjects.Add(ObjectPath, *ObjectMuteData.MuteSetting);
				break;
			case EMuteState::ExplicitlyUnmuted:
				Response.ExplicitlyUnmutedObjects.Add(ObjectPath, *ObjectMuteData.MuteSetting);
				break;
			case EMuteState::ImplicitlyMuted:
				Response.ImplicitlyMutedObjects.Add(ObjectPath);
				break;
			case EMuteState::ImplicitlyUnmuted:
				Response.ImplicitlyUnmutedObjects.Add(ObjectPath);
				break;
			}
		};

		if (Request.WantsAllObjects())
		{
			for (const TPair<FSoftObjectPath, FMuteData>& Pair : MuteState)
			{
				AddObjectToResponse(Pair.Key, Pair.Value);
			}
		}
		else
		{
			for (const FSoftObjectPath& QueriedPath : Request.QueriedObjects)
			{
				if (const FMuteData* ObjectMuteData = MuteState.Find(QueriedPath))
				{
					AddObjectToResponse(QueriedPath, *ObjectMuteData);
				}
			}
		}
		
		return EConcertSessionResponseCode::Success;
	}

	EConcertSessionResponseCode FMuteManager::HandleChangeMuteStateRequest(
		const FConcertSessionContext& Context,
		const FConcertReplication_ChangeMuteState_Request& Request,
		FConcertReplication_ChangeMuteState_Response& Response
		)
	{
		LogNetworkMessage(CVarLogMuteRequestsAndResponsesOnServer, Request, [this, &Context](){ return GetClientName(Session, Context.SourceEndpointId); });
		ON_SCOPE_EXIT{ LogNetworkMessage(CVarLogMuteRequestsAndResponsesOnServer, Response, [this, &Context](){ return GetClientName(Session, Context.SourceEndpointId); }); };
		
		if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldAllowGlobalMuting))
		{
			return EConcertSessionResponseCode::Failed;
		}

		if (!ValidateRequestInternal(Request, Response))
		{
			Response.ErrorCode = EConcertReplicationMuteErrorCode::Rejected;
			return EConcertSessionResponseCode::Success;
		}

		Response.ErrorCode = EConcertReplicationMuteErrorCode::Accepted;
		ApplyRequestInternal(Request);
		
		// Sync control will now 1. generate a sync control we can embed in the response, and 2. send a network event to all other clients.
		// The embedded sync control will contain only those objects which changed sync control
		// Fyi, the new sync control may contain more objects than were specified in the request since parent objects can also cause their subobjects' mute state to change.
		check(OnRefreshSyncControlAndSendToAllClientsExceptDelegate.IsBound());
		Response.SyncControl = OnRefreshSyncControlAndSendToAllClientsExceptDelegate.Execute(Context.SourceEndpointId);
#if UE_BUILD_DEBUG // In debug builds so we don't tank performance
		checkf(!Response.SyncControl.DoesAtLeastOneObjectLoseSyncControl(), TEXT("Clients are supposed to infer that objects lose sync control. We only tell them which objects gained sync control."));
#endif

		OnMuteRequestAppliedDelegate.Broadcast(Context.SourceEndpointId, Request);
		return EConcertSessionResponseCode::Success;
	}

	bool FMuteManager::ValidateRequestInternal(const FConcertReplication_ChangeMuteState_Request& Request, FConcertReplication_ChangeMuteState_Response& Response) const
	{
		const auto Reject = [&Response](const FSoftObjectPath& ObjectPath){ Response.RejectionReasons.Add(ObjectPath); };
		return ValidateRequestInternal(Request, FObjectHierarchyAdapter(ServerObjectCache), Reject);
	}

	bool FMuteManager::ValidateRequestInternal(
		const FConcertReplication_ChangeMuteState_Request& Request, 
		const IMuteValidationObjectHierarchy& ObjectCache,
		TFunctionRef<void(const FSoftObjectPath& ObjectPath)> OnRejection
		) const
	{
		if (EnumHasAnyFlags(Request.Flags, EConcertReplicationMuteRequestFlags::ClearMuteState) && !Request.ObjectsToUnmute.IsEmpty())
		{
			return false;
		}
		
		bool bIsValidRequest = true;
		const auto Reject = [&OnRejection, &bIsValidRequest](const FSoftObjectPath& ObjectPath){ OnRejection(ObjectPath); bIsValidRequest = false; };
		const auto SharedValidateSetting = [this, &Reject, &ObjectCache](const FSoftObjectPath& Object, const FConcertReplication_ObjectMuteSetting& Setting)
		{
			const bool bIsObjectReferenced = ObjectCache.IsObjectReferencedDirectly(Object);
			const bool bAppliesToSubobjects = ConcertSyncCore::AffectSubobjects(Setting.Flags);
			if (!bIsObjectReferenced && !bAppliesToSubobjects)
			{
				Reject(Object);
				return;
			}

			const bool bHasSubobjects = ObjectCache.HasChildren(Object);
			if (!bIsObjectReferenced && bAppliesToSubobjects && !bHasSubobjects)
			{
				Reject(Object);
			}
		};
		
		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& ToMute : Request.ObjectsToMute)
		{
			SharedValidateSetting(ToMute.Key, ToMute.Value);

			// Cannot mute & unmute at the same time
			if (Request.ObjectsToUnmute.Contains(ToMute.Key))
			{
				Reject(ToMute.Key);
			}
		}
		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& ToUnmute : Request.ObjectsToUnmute)
		{
			// If something is muted, the user can always unmute it. If it is already unmuted, it's still valid (but a no-op).
			if (!MuteState.Contains(ToUnmute.Key))
			{
				SharedValidateSetting(ToUnmute.Key, ToUnmute.Value);
			}
		}
		
		return bIsValidRequest;
	}

	void FMuteManager::ApplyRequestInternal(const FConcertReplication_ChangeMuteState_Request& Request)
	{
		const bool bResetState = EnumHasAnyFlags(Request.Flags, EConcertReplicationMuteRequestFlags::ClearMuteState);
		UE_CLOGF(bResetState, LogConcert, Log, "Explicitly muting %d and unmuting %d objects.", Request.ObjectsToMute.Num(), Request.ObjectsToUnmute.Num());
		UE_CLOGF(!bResetState, LogConcert, Log, "Explicitly muting %d and unmuting %d objects. Resetting mute state before.", Request.ObjectsToMute.Num(), Request.ObjectsToUnmute.Num());
		
		if (bResetState)
		{
			MuteState.Empty(Request.ObjectsToMute.Num());
		}
		
		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& ToMute : Request.ObjectsToMute)
		{
			const FSoftObjectPath& ObjectToMute = ToMute.Key;
			const FConcertReplication_ObjectMuteSetting& MuteSetting = ToMute.Value;

			// Muting object again with same parameters is a valid: not work to do if so.
			const bool bAffectsChildren = ConcertSyncCore::AffectSubobjects(MuteSetting.Flags);
			{
				const FMuteData* ObjectMuteState = MuteState.Find(ObjectToMute);
				const bool bAlreadyAtSetting = ObjectMuteState && ObjectMuteState->MuteSetting && ObjectMuteState->MuteSetting == ToMute.Value;
				if (bAlreadyAtSetting)
				{
					continue;
				}

				// If the request is changing the flag from IncludeSubobjects -> None, the children need their implicit state updated.
				const bool bPreviousAffectedChildren = ObjectMuteState && ObjectMuteState->AffectsSubobjects(); 
				if (!bAffectsChildren && bPreviousAffectedChildren)
				{
					RemoveMuteState(ObjectToMute);
				}
				// ObjectMuteState may now be invalid due to possible removal
			}

			const FMuteData NewMuteData
			{
				MuteSetting,
				EMuteState::ExplicitlyMuted
			};
			MuteState.Add(ObjectToMute, NewMuteData);

			// Children may need their implicit state updated
			if (bAffectsChildren)
			{
				UpdateImplicitStateUnder(ObjectToMute, EMuteState::ImplicitlyMuted);
			}
		}

		for (const TPair<FSoftObjectPath, FConcertReplication_ObjectMuteSetting>& ToMute : Request.ObjectsToUnmute)
		{
			const FSoftObjectPath& ObjectToUnmute = ToMute.Key;
			const FConcertReplication_ObjectMuteSetting& MuteSetting = ToMute.Value;
			
			const auto[AffectingParent, ParentMuteState] = FindAffectingParentState(ObjectToUnmute);
			// ParentMuteState can either be:
			//	- EMuteState::ExplicitlyMuted: all children should be muted
			//	- EMuteState::ExplicitlyUnmuted: by invariant, there must be a parent that is ExplicitlyMuted with EConcertReplicationMuteFlags::ObjectAndSubobjects.
			// Hence in both cases, children need to be updated to be marked as implicitly muted.
			if (ParentMuteState)
			{
				MuteState.Add(ObjectToUnmute, { MuteSetting, EMuteState::ExplicitlyUnmuted });
				if (ConcertSyncCore::AffectSubobjects(MuteSetting.Flags))
				{
					UpdateImplicitStateUnder(ObjectToUnmute, EMuteState::ImplicitlyUnmuted);
				}
			}
			else
			{
				// If no parent state affects ObjectToUnmute, the entire subtree must be removed.
				FMuteData MuteData;
				const bool bHadData = MuteState.RemoveAndCopyValue(ObjectToUnmute, MuteData);
				if (bHadData && MuteData.AffectsSubobjects())
				{
					ClearAllChildStateUnder(ObjectToUnmute);
				}
			}
		}
	}

	template<CObjectProcessable TCallback>
	void FMuteManager::RemoveMuteState(const FSoftObjectPath& Object, TCallback&& OnRemoved)
	{
		const FMuteData* ObjectMuteData = MuteState.Find(Object);
		if (!ObjectMuteData)
		{
			return;
		}
		
		// If an Object affects its subobjects...
		if (ObjectMuteData->AffectsSubobjects())
		{
			const auto[AffectingParent, ParentMuteData] = FindAffectingParentState(Object);
			if (ParentMuteData)
			{
				// ... and has a parent also affecting subobjects, then all of Object's children are now affected by that parent.
				const EMuteState ImplicitNewChildState = ParentMuteData->State == EMuteState::ExplicitlyMuted ? EMuteState::ImplicitlyMuted : EMuteState::ImplicitlyUnmuted;
				UpdateImplicitStateUnder(Object, ImplicitNewChildState);
			}
			else
			{
				// ... and does not have any parent affecting subobjects, then clear all child state
				ClearAllChildStateUnder(Object, OnRemoved);
			}
		}

		MuteState.Remove(Object);
		OnRemoved(Object);
	}

	void FMuteManager::UpdateImplicitStateUnder(const FSoftObjectPath& Parent, EMuteState NewImplicitState)
	{
		check(NewImplicitState == EMuteState::ImplicitlyMuted || NewImplicitState == EMuteState::ImplicitlyUnmuted);
		
		ServerObjectCache.TraverseTopToBottom([this, NewImplicitState](const ConcertSyncCore::FChildRelation& Relation)
		{
			FMuteData* SubobjectMuteState = MuteState.Find(Relation.Child.Object);
			if (!SubobjectMuteState)
			{
				MuteState.Add(Relation.Child.Object, { {}, NewImplicitState });
			}
			else if (SubobjectMuteState->MuteSetting
				&& ConcertSyncCore::AffectSubobjects(SubobjectMuteState->MuteSetting->Flags))
			{
				return ConcertSyncCore::ETreeTraversalBehavior::SkipSubtree;
			}
			else
			{
				SubobjectMuteState->State = NewImplicitState;
			}
					
			return ConcertSyncCore::ETreeTraversalBehavior::Continue;
		}, Parent);
	}
	
	template<CObjectProcessable TCallback>
	void FMuteManager::ClearAllChildStateUnder(const FSoftObjectPath& ParentObject, TCallback&& OnRemoved)
	{
		ServerObjectCache.TraverseTopToBottom([this, &OnRemoved](const ConcertSyncCore::FChildRelation& Relation)
		{
			FMuteData* SubobjectMuteState = MuteState.Find(Relation.Child.Object);
			if (!SubobjectMuteState)
			{
				return ConcertSyncCore::ETreeTraversalBehavior::Continue;
			}

			MuteState.Remove(Relation.Child.Object);
			OnRemoved(Relation.Child.Object);
			return ConcertSyncCore::ETreeTraversalBehavior::Continue;
		}, ParentObject);
	}
	
	TPair<FSoftObjectPath, const FMuteManager::FMuteData*> FMuteManager::FindAffectingParentState(const FSoftObjectPath& Subobject) const
	{
		for (ConcertSyncCore::FObjectPathOuterIterator It(Subobject); It; ++It)
		{
			const FSoftObjectPath& Parent = *It;
			if (const FMuteData* ParentMuteState = MuteState.Find(Parent)
				; ParentMuteState
				// Does it have the IncludeSubobjects flag
				&& ParentMuteState->MuteSetting && ConcertSyncCore::AffectSubobjects(ParentMuteState->MuteSetting->Flags)
				// Only explicit objects should have the IncludeSubobjects flag
				&& ensure(ParentMuteState->IsExplicit()))
			{
				return { Parent, ParentMuteState };
			}
		}
		
		return {};
	}
}
