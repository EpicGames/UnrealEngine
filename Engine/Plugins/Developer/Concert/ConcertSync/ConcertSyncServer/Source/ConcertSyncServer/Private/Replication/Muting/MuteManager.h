// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessages.h"
#include "ConcertSyncSessionFlags.h"
#include "IConcertSessionHandler.h"
#include "Replication/Messages/Muting.h"

#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

class IConcertSession;
struct FConcertReplication_ChangeStream_Request;
struct FConcertReplicationStream;
struct FConcertObjectInStreamID;

namespace UE::ConcertSyncCore { class FReplicatedObjectHierarchyCache; }
namespace UE::ConcertSyncServer::Replication {
	class IMuteValidationObjectHierarchy;
	class IRegistrationEnumerator; }

namespace UE::ConcertSyncServer::Replication
{
	/** Invocable with signature void([](const FConcertObjectInStreamID&){}) which calls the passed in lambda */
	template<typename TEnumerateObjects>
	concept CObjectEnumeratable = requires(TEnumerateObjects EnumerateObjects)
	{
		EnumerateObjects([](const FConcertObjectInStreamID&){});
	};
	
	/** Invocable with signature void(FSoftObjectPath) */
	template<typename TLambda>
	concept CObjectProcessable = std::is_invocable_v<TLambda, const FSoftObjectPath&>;
	
	/**
	 * Manages the session's replication mute state.
	 * Handles FConcertReplication_ChangeMuteState_Request and FConcertReplication_QueryMuteState_Request requets.
	 */
	class FMuteManager : public FNoncopyable
	{
	public:

		using FOnSyncControlChange = TFunctionRef<void(const FGuid& ClientId, FConcertReplication_ChangeSyncControl&& SyncControlChange)>;

		DECLARE_DELEGATE_OneParam(FOnMuteStateChangedByClient, const FGuid& ClientId);
		DECLARE_DELEGATE_RetVal_OneParam(FConcertReplication_ChangeSyncControl, FRefreshSyncControlForMuteChange, const FGuid& ClientId);
		DECLARE_DELEGATE_OneParam(FRefreshSyncControlForClients, const FOnSyncControlChange& OnSyncControlChange);
		
		/** Notifies manager about applied event. Generates mute activity. */
		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMuteRequestApplied, const FGuid& ClientId, const FConcertReplication_ChangeMuteState_Request& Request);
		
		enum class EMuteState : uint8
		{
			/** Object is muted */
			ExplicitlyMuted,
			/** A parent object is ExplicitlyMuted and has EConcertReplicationMuteFlags::ObjectAndSubobjects set but is object and its children are supposed to be implicitly unmuted.  */
			ExplicitlyUnmuted,
			
			/** Object is muted because one of its parent objects is ExplicitlyMuted and EConcertReplicationMuteFlags::ObjectAndSubobjects set. */
			ImplicitlyMuted,
			/** Object is unmuted because one of its parent objects is ExplicitlyUnmuted and has EConcertReplicationMuteFlags::ObjectAndSubobjects set. */
			ImplicitlyUnmuted,
		};

		FMuteManager(
			IConcertSession& InSession UE_LIFETIMEBOUND,
			const ConcertSyncCore::FReplicatedObjectHierarchyCache& ServerObjectCache UE_LIFETIMEBOUND,
			EConcertSyncSessionFlags InSessionFlags
			);
		~FMuteManager();

		/** @return Whether the object is globally muted. */
		bool IsMuted(const FSoftObjectPath& Object) const;

		/** @return The mute state of Object. If it was not explicitly muted and no parent object affects this subobject, then the optional is unset. */
		TOptional<EMuteState> GetMuteState(const FSoftObjectPath& Object) const;
		/** @return The mute setting of Object, if it is explicitly set (i.e. not affected by a parent object). */
		TOptional<FConcertReplication_ObjectMuteSetting> GetExplicitMuteSetting(const FSoftObjectPath& Object) const;
		
		/**
		 * Checks whether Request would be valid to apply, optionally providing a future server state.
		 * @param Request The request to validate
		 * @param OverrideServerObjectCache The server cache to use. Defaults to the internal one if left nullptr. Useful for passing in a future server state.
		 * @param OnRejection Called for every rejected object.
		 */
		bool ValidateRequest(
			const FConcertReplication_ChangeMuteState_Request& Request,
			const IMuteValidationObjectHierarchy* OverrideServerObjectCache = nullptr,
			TFunctionRef<void(const FSoftObjectPath& ObjectPath)> OnRejection = [](const FSoftObjectPath&){}
			) const;
		
		/**
		 * This validates and then applies Request.
		 * This version does not send any sync control updates to other clients. You must update them yourself.
		 * 
		 * @param Request The request to apply
		 * @param OnSyncControlChange Called for every sync control change that is made.
		 * @return Whether the request was applied.
		 */
		bool ApplyRequestAndEnumerateSyncControl(
			const FConcertReplication_ChangeMuteState_Request& Request,
			const FOnSyncControlChange& OnSyncControlChange
			);
		/**
		 * Applies Request as if it was sent by EndpointId.
		 * @return The sync control that the client identified by EndpointId has gained. Does NOT include the sync control they lost.
		 */
		FConcertReplication_ChangeSyncControl ApplyManualRequest(const FGuid& EndpointId, const FConcertReplication_ChangeMuteState_Request& Request);
		
		/** Called right after objects have been unregistered from ClientId's streams. */
		void PostApplyStreamChange(const FGuid& ClientId, TConstArrayView<FConcertObjectInStreamID> AddedObjects, TConstArrayView<FConcertObjectInStreamID> RemovedObjects);
		/** Called by FConcertServerReplicationManager when client leaves replication. */
		void OnPostClientLeft(const TArray<FConcertReplicationStream>& ClientStreams);

		/** Updates sync control if needed in response to a client changing their mute state indirectly (e.g. because client removed object from their stream). */
		FOnMuteStateChangedByClient& OnRefreshSyncControlForIndirectMuteChange() { return OnMuteStateChangedDelegate; }
		/** Updates sync control for all clients, sends an update to all clients but ClientId, and returns the sync control to embed into the mute response. */
		FRefreshSyncControlForMuteChange& OnRefreshSyncControlAndSendToAllClientsExcept() { return OnRefreshSyncControlAndSendToAllClientsExceptDelegate; }
		/** Updates sync control for all clients but does not send any update to any clients. The passed in callback receives all sync control changes for the clients. */
		FRefreshSyncControlForClients& OnRefreshSyncControlButSkipSendingToClients() { return OnRefreshSyncControlButSkipSendingToClientsDelegate; }
		
		FOnMuteRequestApplied& OnMuteRequestApplied() { return OnMuteRequestAppliedDelegate; }
		
	private:

		/** The session to receive requests on. */
		IConcertSession& Session;
		/** Keeps track of all objects that are in client streams. Allows efficient traversing of subobject hierarchy. */
		const ConcertSyncCore::FReplicatedObjectHierarchyCache& ServerObjectCache;

		/** If ShouldAllowGlobalMuting is not set, the requests are not served. */
		const EConcertSyncSessionFlags SessionFlags;

		/** Broadcasts after the mute state has been changed. */
		FOnMuteStateChangedByClient OnMuteStateChangedDelegate;
		/**
		 * Delegate into FSyncControlManager
		 * 
		 * Updates sync control for all clients, sends an update to all clients but ClientId, and returns the sync control to embed into the mute response.
		 * The sync control will only contain RestrictToObjects.
		 */
		FRefreshSyncControlForMuteChange OnRefreshSyncControlAndSendToAllClientsExceptDelegate;
		/** Delegate into FSyncControlManager.  */
		FRefreshSyncControlForClients OnRefreshSyncControlButSkipSendingToClientsDelegate;
		
		/** Broadcasts after a mute request has been applied. */
		FOnMuteRequestApplied OnMuteRequestAppliedDelegate;
		
		struct FMuteData
		{
			/** Only set if the object was explicitly muted or unmuted. */
			TOptional<FConcertReplication_ObjectMuteSetting> MuteSetting;
			
			EMuteState State;

			bool IsExplicit() const { return MuteSetting.IsSet(); }
			bool IsImplicit() const { return !IsExplicit(); }
			bool AffectsSubobjects() const { return MuteSetting && ConcertSyncCore::AffectSubobjects(MuteSetting->Flags); }
		};
		/**
		 * Holds the mute state of all objects.
		 * 
		 * Objects only show up in this map if they are either explicitly or implicitly affected by a mute setting:
		 * - explicit means that a request listed them to be muted / unmuted
		 * - implicit means that a parent object (i.e. an outer) is explicitly muted or unmuted.
		 * Hence, if an object is not in this map it means that neither it nor any of its parents are explicitly/implicitly muted nor unmuted.
		 */
		TMap<FSoftObjectPath, FMuteData> MuteState;
		
		/** For each added object, checks whether it is a subobject of an object that is muted with the EConcertReplicationMuteFlags::ObjectAndSubobjects setting. */
		void TrackAddedSubobjectsForImplicitMuting(TConstArrayView<FConcertObjectInStreamID> AddedObjects);
		/** Calls RemoveMuteState on every object that is unreferenced by client streams (directly or indirectly via a subobject). */
		void UnmuteObjectsIfUnreferenced(TConstArrayView<FConcertObjectInStreamID> RemovedObjects);
		/** Calls RemoveMuteState if RemovedObject is unreferenced by client streams (directly or indirectly via a subobject). */
		template<CObjectProcessable TCallback>
		void UnmuteObjectIfUnreferenced(const FSoftObjectPath& RemovedObject, TCallback&& OnRemoved);
		void UnmuteObjectIfUnreferenced(const FSoftObjectPath& RemovedObject) { UnmuteObjectIfUnreferenced(RemovedObject, [](const FSoftObjectPath& ObjectPath){}); }

		/** Handles requests to query mute states. */
		EConcertSessionResponseCode HandleQueryMuteStateRequest(const FConcertSessionContext& Context, const FConcertReplication_QueryMuteState_Request& Request, FConcertReplication_QueryMuteState_Response& Response);
		/** Handles requests to change the mute states */
		EConcertSessionResponseCode HandleChangeMuteStateRequest(const FConcertSessionContext& Context, const FConcertReplication_ChangeMuteState_Request& Request, FConcertReplication_ChangeMuteState_Response& Response);

		/** Checks whether Request is valid to apply. */
		bool ValidateRequestInternal(const FConcertReplication_ChangeMuteState_Request& Request, FConcertReplication_ChangeMuteState_Response& Response) const;
		/** Checks whether Request is valid to apply. */
		bool ValidateRequestInternal(const FConcertReplication_ChangeMuteState_Request& Request, const IMuteValidationObjectHierarchy& ObjectCache, TFunctionRef<void(const FSoftObjectPath& ObjectPath)> OnRejection) const;
		/** Updates the internal state (assuming it is a valid request - call ValidateRequest before). */
		void ApplyRequestInternal(const FConcertReplication_ChangeMuteState_Request& Request);

		/** Removes all mute state for Object and transitively updates subobjects if applicable. */
		template<CObjectProcessable TCallback>
		void RemoveMuteState(const FSoftObjectPath& Object, TCallback&& OnRemoved);
		void RemoveMuteState(const FSoftObjectPath& Object) { RemoveMuteState(Object, [](const FSoftObjectPath&){}); } 
		
		/** Sets the mute state of all subobjects of Parent until reaching a subobject that also has EConcertReplicationMuteFlags::ObjectAndSubobjects set. */
		void UpdateImplicitStateUnder(const FSoftObjectPath& Parent, EMuteState NewImplicitState);
		
		/** Removes all subobject mute state affected by Parent's mute state assuming that ParentObject has no parents with the EConcertReplicationMuteFlags::ObjectAndSubobjects. */
		template<CObjectProcessable TCallback>
		void ClearAllChildStateUnder(const FSoftObjectPath& ParentObject, TCallback&& OnRemoved);
		void ClearAllChildStateUnder(const FSoftObjectPath& ParentObject) { ClearAllChildStateUnder(ParentObject, [](const FSoftObjectPath&){}); }
		
		/** Walks up the object hierarchy and returns the parent that affects ObjectPath, if any. */
		TPair<FSoftObjectPath, const FMuteData*> FindAffectingParentState(const FSoftObjectPath& Subobject) const;
	};
}

