// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMuteValidationObjectHierarchy.h"
#include "Replication/Misc/ReplicatedObjectHierarchyCache.h"

#include "HAL/Platform.h"

namespace UE::ConcertSyncServer::Replication
{
	/** Simple adapter implementation that uses the server's current FReplicatedObjectHierarchyCache state. */
	class FObjectHierarchyAdapter : public IMuteValidationObjectHierarchy
	{
		const ConcertSyncCore::FReplicatedObjectHierarchyCache& ServerCache;
	public:
		
		FObjectHierarchyAdapter(const ConcertSyncCore::FReplicatedObjectHierarchyCache& ServerCache UE_LIFETIMEBOUND)
			: ServerCache(ServerCache)
		{}

		//~ Begin IMuteValidationObjectHierarchy Interface
		virtual bool IsObjectReferencedDirectly(const FSoftObjectPath& ObjectPath, TConstArrayView<FGuid> IgnoredClients) const override { return ServerCache.IsObjectReferencedDirectly(ObjectPath, IgnoredClients); }
		virtual bool HasChildren(const FSoftObjectPath& Object) const override { return ServerCache.HasChildren(Object); }
		//~ Begin IMuteValidationObjectHierarchy Interface
	};
}
