// Copyright Epic Games, Inc. All Rights Reserved.

#include "PredictedStateObjectHierarchy.h"

#include "Algo/AllOf.h"
#include "Replication/Data/ReplicationStream.h"

namespace UE::ConcertSyncServer::Replication
{
	void FPredictedStateObjectHierarchy::AddClients(const TMap<FGuid, FConcertReplicationStreamArray>& Clients)
	{
		for (const TPair<FGuid, FConcertReplicationStreamArray>& Pair : Clients)
		{
			AddClientData(Pair.Key, Pair.Value.Streams);
		}
	}

	void FPredictedStateObjectHierarchy::AddClientData(const FGuid& ClientId, TConstArrayView<FConcertReplicationStream> Streams)
	{
		for (const FConcertReplicationStream& Stream : Streams)
		{
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				const FSoftObjectPath& ObjectPath = Pair.Key;
				Hierarchy.AddObject(ObjectPath);
				ObjectReferencingClients.FindOrAdd(ObjectPath).AddUnique(ClientId);
			}
		}
	}

	bool FPredictedStateObjectHierarchy::IsObjectReferencedDirectly(const FSoftObjectPath& ObjectPath, TConstArrayView<FGuid> IgnoredClients) const
	{
		const TArray<FGuid>* ReferencingClients = ObjectReferencingClients.Find(ObjectPath);
		return ReferencingClients && Algo::AllOf(*ReferencingClients, [&IgnoredClients](const FGuid& ClientId){ return !IgnoredClients.Contains(ClientId); });
	}

	bool FPredictedStateObjectHierarchy::HasChildren(const FSoftObjectPath& Object) const
	{
		return Hierarchy.HasChildren(Object);
	}
}
