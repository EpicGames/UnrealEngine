// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMuteValidationObjectHierarchy.h"
#include "Misc/ObjectPathHierarchy.h"
#include "Replication/ConcertReplicationClient.h"
#include "Replication/Data/ReplicationStreamArray.h"

#include <type_traits>


struct FConcertReplicationStream;

namespace UE::ConcertSyncServer::Replication
{
	class IRegistrationEnumerator;

	/**
	 * Implementation of IMuteValidationObjectHierarchy which allows you to add arbitrary client state.
	 * Primarily used to validate a hierarchy that clients WILL have in the future.
	 */
	class FPredictedStateObjectHierarchy : public IMuteValidationObjectHierarchy
	{
	public:

		/** Adds clients from a map binding client id to stream content. */
		void AddClients(const TMap<FGuid, FConcertReplicationStreamArray>& Clients);
		/** Adds only those entries that pass ShouldIncludeFilter returns true on. */
		template<typename TFilterLambda> requires std::is_invocable_r_v<bool, TFilterLambda, const FGuid&>
		void AddClients(const TMap<FGuid, TUniquePtr<FConcertReplicationClient>>& Clients, TFilterLambda&& ShouldIncludeFilter);
		
		/** Adds the streams a client will have. */
		void AddClientData(const FGuid& ClientId, TConstArrayView<FConcertReplicationStream> Streams);
		
		//~ Begin IMuteValidationObjectHierarchy Interface
		virtual bool IsObjectReferencedDirectly(const FSoftObjectPath& ObjectPath, TConstArrayView<FGuid> IgnoredClients) const override;
		virtual bool HasChildren(const FSoftObjectPath& Object) const override;
		//~ End IMuteValidationObjectHierarchy Interface

	private:

		/** The hierarchy built during construction. */
		ConcertSyncCore::FObjectPathHierarchy Hierarchy;

		/** Maps objects to the clients referencing them. */
		TMap<FSoftObjectPath, TArray<FGuid>> ObjectReferencingClients;
	};

	template <typename TFilterLambda> requires std::is_invocable_r_v<bool, TFilterLambda, const FGuid&>
	void FPredictedStateObjectHierarchy::AddClients(const TMap<FGuid, TUniquePtr<FConcertReplicationClient>>& Clients, TFilterLambda&& ShouldIncludeFilter)
	{
		for (const TPair<FGuid, TUniquePtr<FConcertReplicationClient>>& Pair : Clients)
		{
			if (ShouldIncludeFilter(Pair.Key))
			{
				AddClientData(Pair.Key, Pair.Value->GetStreamDescriptions());
			}
		}
	}
}
