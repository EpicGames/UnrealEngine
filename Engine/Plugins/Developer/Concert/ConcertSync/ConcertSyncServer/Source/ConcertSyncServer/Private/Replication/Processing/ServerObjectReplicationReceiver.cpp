// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerObjectReplicationReceiver.h"

#include "ConcertLogGlobal.h"
#include "IConcertSessionHandler.h"
#include "Replication/AuthorityManager.h"
#include "Replication/SyncControlManager.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/Messages/ObjectReplication.h"

namespace UE::ConcertSyncServer::Replication
{
	FServerObjectReplicationReceiver::FServerObjectReplicationReceiver(
		const FAuthorityManager& AuthorityManager,
		const FSyncControlManager& SyncControlManager,
		IConcertSession& Session,
		ConcertSyncCore::FObjectReplicationCache& ReplicationCache
		)
		: FObjectReplicationReceiver(Session, ReplicationCache)
		, AuthorityManager(AuthorityManager)
		, SyncControlManager(SyncControlManager)
	{}

	bool FServerObjectReplicationReceiver::ShouldAcceptObject(
		const FConcertSessionContext& SessionContext,
		const FConcertReplication_StreamReplicationEvent& StreamEvent,
		const FConcertReplication_ObjectReplicationEvent& ObjectEvent
		) const
	{
		const FConcertReplicatedObjectId ReplicatedObjectInfo { { StreamEvent.StreamId, ObjectEvent.ReplicatedObject }, SessionContext.SourceEndpointId };
		
		const bool bHasAuthority = AuthorityManager.HasAuthorityToChange(ReplicatedObjectInfo);
		const bool bHasSyncControl = SyncControlManager.HasSyncControl(ReplicatedObjectInfo);
		UE_CLOGF(!bHasAuthority, LogConcert, Verbose, "Dropping %ls because the client does not have authority over it.", *ReplicatedObjectInfo.ToString());
		UE_CLOGF(!bHasSyncControl, LogConcert, Verbose, "Dropping %ls because the client does not have sync control over it.", *ReplicatedObjectInfo.ToString());

		// Note: Having sync control logically implies that the sender should have authority but we check both conditions here for completeness.
		return bHasAuthority && bHasSyncControl;
	}
}
