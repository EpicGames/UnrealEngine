// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Processing/ObjectReplicationReceiver.h"

#include "HAL/Platform.h"

namespace UE::ConcertSyncServer::Replication
{
	class FAuthorityManager;
	class FSyncControlManager;
	
	/** Rejects changes to objects that the sending client does not have authority over. */
	class FServerObjectReplicationReceiver : public ConcertSyncCore::FObjectReplicationReceiver
	{
	public:

		FServerObjectReplicationReceiver(
			const FAuthorityManager& AuthorityManager UE_LIFETIMEBOUND,
			const FSyncControlManager& SyncControlManager UE_LIFETIMEBOUND,
			IConcertSession& Session UE_LIFETIMEBOUND,
			ConcertSyncCore::FObjectReplicationCache& ReplicationCache UE_LIFETIMEBOUND
			);

	protected:

		//~ Begin FObjectReplicationReceiver Interface
		virtual bool ShouldAcceptObject(const FConcertSessionContext& SessionContext, const FConcertReplication_StreamReplicationEvent& StreamEvent, const FConcertReplication_ObjectReplicationEvent& ObjectEvent) const override;
		//~ End FObjectReplicationReceiver Interface

	private:

		/** Used to determine whether a client has authority over objects. */
		const FAuthorityManager& AuthorityManager;
		/** Used to determine whether any client is listening to an incoming object. */
		const FSyncControlManager& SyncControlManager;
	};
}


