// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerReplicationDataQueuer.h"

#include "Replication/ConcertReplicationClient.h"

namespace UE::ConcertSyncServer::Replication
{
	TSharedRef<FServerReplicationDataQueuer> FServerReplicationDataQueuer::Make(
		const FGuid& OwningClientEndpointId,
		ConcertSyncCore::FObjectReplicationCache& InReplicationCache
		)
	{
		TSharedRef<FServerReplicationDataQueuer> Result = MakeShared<FServerReplicationDataQueuer>(OwningClientEndpointId);
		Result->BindToCache(InReplicationCache);
		return Result;
	}

	bool FServerReplicationDataQueuer::WantsToAcceptObject(const FConcertReplicatedObjectId& Object) const
	{
		// Do not send back the data to the client that generated it
		const bool bWasSentByThisClient = OwningClientEndpointId == Object.SenderEndpointId;
		// TODO DP: For now accept all objects from other clients. In the future, clients can specify what data they want to receive using client attributes.
		return !bWasSentByThisClient;
	}

	FServerReplicationDataQueuer::FServerReplicationDataQueuer(const FGuid& OwningClientEndpointId)
		: OwningClientEndpointId(OwningClientEndpointId)
	{}
}
