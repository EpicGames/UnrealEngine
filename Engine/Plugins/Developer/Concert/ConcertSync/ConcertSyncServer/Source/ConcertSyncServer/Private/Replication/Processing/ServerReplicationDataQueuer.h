// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Misc/Guid.h"
#include "Replication/Processing/ReplicationDataQueuer.h"
#include "Templates/SharedPointer.h"

namespace UE::ConcertSyncServer::Replication
{
	class FConcertReplicationClient;
	
	/**
	 * Queues events for a specific client.
	 * TODO: Filters events by matching client and stream attributes.
	 */
	class FServerReplicationDataQueuer : public ConcertSyncCore::FReplicationDataQueuer
	{
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;
	public:
		
		static TSharedRef<FServerReplicationDataQueuer> Make(const FGuid& OwningClientEndpointId, ConcertSyncCore::FObjectReplicationCache& InReplicationCache UE_LIFETIMEBOUND);
		
		//~ Begin IReplicationCacheUser Interface
		virtual bool WantsToAcceptObject(const FConcertReplicatedObjectId& Object) const override;
		//~ End IReplicationCacheUser Interface

	private:

		FServerReplicationDataQueuer(const FGuid& OwningClientEndpointId);

		/** The client for which this FServerReplicationDataQueuer exists. */
		const FGuid OwningClientEndpointId;
	};
}

