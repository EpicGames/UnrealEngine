// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertServerReplicationManager.h"
#include "ReplicationWorkspace.h"
#include "Replication/IConcertServerReplicationManager.h"

#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"

class IConcertServerSession;

/**
 * Exposes functions that are required for testing.
 * 
 * These functions are technically exported but conceptually not part of the public interface
 * and should only be used for the purpose of automated testing.
 */
namespace UE::ConcertSyncServer::TestInterface
{
	CONCERTSYNCSERVER_API TSharedRef<Replication::IConcertServerReplicationManager> CreateServerReplicationManager(
		TSharedRef<IConcertServerSession> InLiveSession,
		Replication::IReplicationWorkspace& InWorkspace,
		EConcertSyncSessionFlags InSessionFlags
		)
	{
		return MakeShared<Replication::FConcertServerReplicationManager>(InLiveSession, InWorkspace, InSessionFlags);
	}

	CONCERTSYNCSERVER_API TSharedRef<Replication::IReplicationWorkspace> CreateReplicationWorkspace(
		FConcertSyncSessionDatabase& Database,
		TFunction<TOptional<FConcertSessionClientInfo>(const FGuid& EndpointId)> FindSessionClient,
		TFunction<bool(const FGuid& ClientId)> ShouldIgnoreClientActivityOnRestore
		)
	{
		return MakeShared<FReplicationWorkspace>(
			Database,
			FFindSessionClient::CreateLambda([FindSessionClient = MoveTemp(FindSessionClient)](const FGuid& EndpointId)
				{ return FindSessionClient(EndpointId); }),
			FShouldIgnoreClientActivityOnRestore::CreateLambda([ShouldIgnoreClientActivityOnRestore = MoveTemp(ShouldIgnoreClientActivityOnRestore)](const FGuid& EndpointId)
				{ return ShouldIgnoreClientActivityOnRestore(EndpointId); })
			);
	}
}
