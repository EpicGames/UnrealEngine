// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationWorkspace.h"

#include "ConcertLogGlobal.h"
#include "ConcertSyncSessionDatabase.h"
#include "Replication/Messages/ReplicationActivity.h"

namespace UE::ConcertSyncServer
{
	FReplicationWorkspace::FReplicationWorkspace(
		FConcertSyncSessionDatabase& Database,
		FFindSessionClient InFindSessionClientDelegate,
		FShouldIgnoreClientActivityOnRestore InShouldIgnoreClientActivityOnRestoreDelegate
		)
		: Database(Database)
		, FindSessionClientDelegate(MoveTemp(InFindSessionClientDelegate))
		, ShouldIgnoreClientActivityOnRestoreDelegate(MoveTemp(InShouldIgnoreClientActivityOnRestoreDelegate))
	{
		check(FindSessionClientDelegate.IsBound() && ShouldIgnoreClientActivityOnRestoreDelegate.IsBound());
	}

	TOptional<int64> FReplicationWorkspace::ProduceReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationEvent& EventData)
	{
		FConcertSyncReplicationActivity Activity;
		Activity.EndpointId = EndpointId;
		Activity.EventData = EventData;
		Activity.EventSummary.SetTypedPayload(FConcertSyncReplicationActivitySummary::CreateSummaryForEvent(Activity.EventData));
		Activity.bIgnored = ShouldIgnoreClientActivityOnRestoreDelegate.Execute(EndpointId);
		
		int64 ActivityId = 0;
		int64 EventId = 0;
		const bool bSuccess = Database.AddReplicationActivity(Activity, ActivityId, EventId);
		// This allows FConcertServerWorkspace to send the activity to other clients
		OnAddReplicationActivityDelegate.Broadcast(ActivityId, bSuccess);

		return bSuccess ? ActivityId : TOptional<int64>{};
	}

	bool FReplicationWorkspace::GetLastReplicationActivityByClient(const FConcertSessionClientInfo& InClientInfo, EConcertSyncReplicationActivityType ActivityType, FConcertSyncReplicationActivity& OutActivity) const
	{
		// Goal: Iterate all endpoints with the same display name. Then, get the latest transaction ID from all of them.
		// This entire approach is suboptimal though: There should be a dedicated SQL prepared statement  to which we give the client and device name.
		// That's not possible though because the endpoints.client_info_data cannot be used to query by SQL as it's a BLOB; this strictly also violates
		// column atomicity of 1NF.
		TArray<FConcertSyncEndpointIdAndData> EndpointsWithSameDeviceName;
		TArray<FConcertSyncEndpointIdAndData> EndpointsWithOtherDeviceName;
		Database.EnumerateEndpoints([&InClientInfo, &EndpointsWithSameDeviceName, &EndpointsWithOtherDeviceName](FConcertSyncEndpointIdAndData&& EndpointIdAndData)
		{
			const FConcertClientInfo& ClientInfo = EndpointIdAndData.EndpointData.ClientInfo;
			const FConcertClientInfo& SearchedClientInfo = InClientInfo.ClientInfo;
			if (ClientInfo.DisplayName == SearchedClientInfo.DisplayName)
			{
				const bool bSameDeviceName = ClientInfo.DeviceName == SearchedClientInfo.DeviceName;
				TArray<FConcertSyncEndpointIdAndData>& ArrayToAddTo = bSameDeviceName ? EndpointsWithSameDeviceName : EndpointsWithOtherDeviceName;
				ArrayToAddTo.Add(EndpointIdAndData);
			}
			return true;
		});

		const TArray<FConcertSyncEndpointIdAndData>& ArrayToSearch = EndpointsWithSameDeviceName.IsEmpty()
			? EndpointsWithOtherDeviceName
			: EndpointsWithSameDeviceName;
		int64 NewestEventId = INDEX_NONE;
		for (const FConcertSyncEndpointIdAndData& ClientData : ArrayToSearch)
		{
			int64 EventId = INDEX_NONE;
			Database.GetReplicationMaxEventIdByClientAndType(ClientData.EndpointId, ActivityType, EventId);
			NewestEventId = FMath::Max(NewestEventId, EventId);
		}

		const bool bGotData = NewestEventId != INDEX_NONE && Database.GetReplicationActivityForEvent(NewestEventId, OutActivity);
		return bGotData;
	}

	bool FReplicationWorkspace::GetReplicationEventById(const int64 ActivityId, FConcertSyncReplicationEvent& OutEvent) const
	{
		return Database.GetReplicationEvent(ActivityId, OutEvent);
	}

	void FReplicationWorkspace::EnumerateReplicationActivities(TFunctionRef<EBreakBehavior(const FConcertSyncReplicationActivity& Activity)> Callback) const
	{
		Database.EnumerateReplicationActivities([&Callback](FConcertSyncReplicationActivity&& Activity)
		{
			return Callback(Activity) == EBreakBehavior::Continue;
		});
	}
}
