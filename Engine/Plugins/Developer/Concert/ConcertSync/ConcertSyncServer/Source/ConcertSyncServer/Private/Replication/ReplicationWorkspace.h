// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IReplicationWorkspace.h"

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"

class FConcertSyncSessionDatabase;

namespace UE::ConcertSyncServer
{
	DECLARE_DELEGATE_RetVal_OneParam(TOptional<FConcertSessionClientInfo>, FFindSessionClient, const FGuid& EndpointId);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldIgnoreClientActivityOnRestore, const FGuid& ClientId);
	
	/**
	 * Implements the replication workspace server-side.
	 * 
	 * At runtime, this is created by FConcertServerWorkspace.
	 * This exists as independent class so it can be unit-tested.
	 */
	class FReplicationWorkspace : public Replication::IReplicationWorkspace
	{
	public:

		FReplicationWorkspace(
			FConcertSyncSessionDatabase& Database UE_LIFETIMEBOUND,
			FFindSessionClient InFindSessionClientDelegate,
			FShouldIgnoreClientActivityOnRestore InShouldIgnoreClientActivityOnRestoreDelegate
			);

		/** Called when an activity is added to the database. */
		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAddReplicationActivity, int64 ActivityId, bool bSuccess);
		FOnAddReplicationActivity& OnAddReplicationActivity() { return OnAddReplicationActivityDelegate; }

		//~ Begin IReplicationWorkspace Interface
		virtual TOptional<int64> ProduceReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationEvent& EventData) override;
		virtual bool GetLastReplicationActivityByClient(const FConcertSessionClientInfo& InClientInfo, EConcertSyncReplicationActivityType ActivityType, FConcertSyncReplicationActivity& OutActivity) const override;
		virtual bool GetReplicationEventById(const int64 ActivityId, FConcertSyncReplicationEvent& OutEvent) const override;
		virtual void EnumerateReplicationActivities(TFunctionRef<EBreakBehavior(const FConcertSyncReplicationActivity& Activity)> Callback) const override;
		//~ End IReplicationWorkspace Interface

	private:

		/** Needed to get replication data. */
		FConcertSyncSessionDatabase& Database;
		/** Needed by GetLastLeaveReplicationActivityByClient to get the most appropriate activity data. */
		const FFindSessionClient FindSessionClientDelegate;
		/** Needed by ProduceClientLeaveReplicationActivity to correctly build the activity data. */
		const FShouldIgnoreClientActivityOnRestore ShouldIgnoreClientActivityOnRestoreDelegate;
		
		FOnAddReplicationActivity OnAddReplicationActivityDelegate;
	};
}

