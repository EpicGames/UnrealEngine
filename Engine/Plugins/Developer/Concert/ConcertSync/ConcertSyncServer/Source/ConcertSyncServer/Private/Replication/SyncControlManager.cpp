// Copyright Epic Games, Inc. All Rights Reserved.

#include "SyncControlManager.h"

#include "AuthorityManager.h"
#include "ConcertServer/Private/ConcertServerSession.h"
#include "Muting/MuteManager.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Messages/SyncControl.h"

namespace UE::ConcertSyncServer::Replication
{
	FSyncControlManager::FSyncControlManager(
		IConcertServerSession& Session,
		FAuthorityManager& AuthorityManager,
		FMuteManager& MuteManager,
		const IRegistrationEnumerator& Getters
		)
		: Session(Session)
		, AuthorityManager(AuthorityManager)
		, MuteManager(MuteManager)
		, Getters(Getters)
	{
		AuthorityManager.OnGenerateSyncControl().BindRaw(this, &FSyncControlManager::OnGenerateSyncControlForAuthorityResponse);
		MuteManager.OnRefreshSyncControlForIndirectMuteChange().BindRaw(this, &FSyncControlManager::OnRefreshSyncControlForIndirectMuteChange);
		MuteManager.OnRefreshSyncControlAndSendToAllClientsExcept().BindRaw(this, &FSyncControlManager::OnRefreshSyncControlForClientMuteChange);
		MuteManager.OnRefreshSyncControlButSkipSendingToClients().BindRaw(this, &FSyncControlManager::OnRefreshSyncControlAndEnumerateWithoutSending);
	}

	FSyncControlManager::~FSyncControlManager()
	{
		Session.OnSessionClientChanged().RemoveAll(this);
		
		if (ensure(AuthorityManager.OnGenerateSyncControl().IsBoundToObject(this)))
		{
			AuthorityManager.OnGenerateSyncControl().Unbind();
		}
		if (ensure(MuteManager.OnRefreshSyncControlForIndirectMuteChange().IsBoundToObject(this)))
		{
			MuteManager.OnRefreshSyncControlForIndirectMuteChange().Unbind();
		}
		if (ensure(MuteManager.OnRefreshSyncControlAndSendToAllClientsExcept().IsBoundToObject(this)))
		{
			MuteManager.OnRefreshSyncControlAndSendToAllClientsExcept().Unbind();
		}
		if (ensure(MuteManager.OnRefreshSyncControlButSkipSendingToClients().IsBoundToObject(this)))
		{
			MuteManager.OnRefreshSyncControlButSkipSendingToClients().Unbind();
		}
	}

	bool FSyncControlManager::HasSyncControl(const FConcertReplicatedObjectId& Object) const
	{
		const FClientData* ClientData = PerClientData.Find(Object.SenderEndpointId);
		return ClientData && ClientData->ObjectsWithSyncControl.Contains(Object);
	}

	const TSet<FConcertObjectInStreamID>* FSyncControlManager::GetClientControlledObjects(const FGuid& ClientId) const
	{
		const FClientData* ClientData = PerClientData.Find(ClientId);
		return ClientData ? &ClientData->ObjectsWithSyncControl : nullptr;
	}

	FConcertReplication_ChangeSyncControl FSyncControlManager::OnGenerateSyncControlForClientJoin(const FGuid& ClientId)
	{
		RefreshAndSendToAllClientsExcept(ClientId);
		return RefreshClientSyncControl(ClientId);
	}

	FConcertReplication_ChangeSyncControl FSyncControlManager::OnGenerateSyncControlForAuthorityResponse(const FGuid& ClientId)
	{
		RefreshAndSendToAllClientsExcept(ClientId);
		const auto ShouldSkipInControlMessage = [](const FConcertObjectInStreamID&, bool bNewState)
		{
			// We skip including states that disable sync control to save on network bandwidth.
			// The receiving client can already predict the objects that lose sync control based on their request.
			// See FConcertReplication_ChangeSyncControl and FConcertReplication_ChangeAuthority_Response::SyncControl documentation.
			return !bNewState;
		};
		return RefreshClientSyncControl(ClientId, ShouldSkipInControlMessage);
	}

	FConcertReplication_ChangeSyncControl FSyncControlManager::OnRefreshSyncControlForClientMuteChange(const FGuid& ClientId)
	{
		RefreshAndSendToAllClientsExcept(ClientId);
		const auto ShouldSkipInControlMessage = [](const FConcertObjectInStreamID&, bool bNewState)
		{
			// We skip including states that disable sync control to save on network bandwidth.
			// The receiving client can already predict the objects that lose sync control based on their request.
			// See FConcertReplication_ChangeMuteState_Request and FConcertReplication_ChangeMuteState_Response::SyncControl documentation.
			return !bNewState;
		};
		return RefreshClientSyncControl(ClientId, ShouldSkipInControlMessage);
	}

	void FSyncControlManager::OnRefreshSyncControlAndEnumerateWithoutSending(const FMuteManager::FOnSyncControlChange& OnSyncControlChange)
	{
		Getters.ForEachReplicationClient([this, &OnSyncControlChange](const FGuid& ClientId)
		{
			OnSyncControlChange(ClientId, RefreshClientSyncControl(ClientId));
			return EBreakBehavior::Continue;
		});
	}

	void FSyncControlManager::HandleClientLeave(const FGuid& LeftClientId)
	{
		// The disconnecting client's objects should be removed
		PerClientData.Remove(LeftClientId);

		// The leaving client may have been the last client to listen for certain object updates
		RefreshAndSendToAllClientsExcept(LeftClientId);
	}
	
	void FSyncControlManager::RefreshAndSendSyncControl(const FGuid& ClientId)
	{
		FConcertReplication_ChangeSyncControl SyncControlChange = RefreshClientSyncControl(ClientId);
		if (!SyncControlChange.IsEmpty())
		{
			Session.SendCustomEvent(SyncControlChange, { ClientId }, EConcertMessageFlags::ReliableOrdered);
		}
	}
	
	FConcertReplication_ChangeSyncControl FSyncControlManager::RefreshClientSyncControl(
		const FGuid& ClientId,
		TFunctionRef<bool(const FConcertObjectInStreamID& Object, bool bNewValue)> ShouldSkipInMessage
		)
	{
		FConcertReplication_ChangeSyncControl SyncControlChange;
		const bool bHadSyncControlBefore = PerClientData.Contains(ClientId);
		FClientData& ClientData = PerClientData.FindOrAdd(ClientId);

		Getters.ForEachStream(ClientId,
			[this, &ClientId, &ShouldSkipInMessage, bHadSyncControlBefore, &ClientData, &SyncControlChange]
			(const FConcertReplicationStream& Stream)
			{
				const FConcertBaseStreamInfo& BaseInfo = Stream.BaseDescription;
				for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : BaseInfo.ReplicationMap.ReplicatedObjects)
				{
					const FConcertObjectInStreamID ObjectId { BaseInfo.Identifier, Pair.Key };
					const FConcertReplicatedObjectId ReplicatedObjectId { ObjectId, ClientId };
					const bool bIsValidForSending = Pair.Value.IsValidForSendingToServer()
						&& AuthorityManager.HasAuthorityToChange(ReplicatedObjectId)
						&& !MuteManager.IsMuted(ReplicatedObjectId.Object)
						&& IsAnyoneInterestedIn(ReplicatedObjectId);
					
					const bool bIsSameAsBefore = ClientData.ObjectsWithSyncControl.Contains(ObjectId) == bIsValidForSending;
					if (bIsValidForSending)
					{
						ClientData.ObjectsWithSyncControl.Add(ObjectId);
					}
					else
					{
						ClientData.ObjectsWithSyncControl.Remove(ObjectId);
					}
					
					const bool bNeedsToIncludeInMessage = !bHadSyncControlBefore || !bIsSameAsBefore;
					if (bNeedsToIncludeInMessage && !ShouldSkipInMessage(ObjectId, bIsValidForSending))
					{
						SyncControlChange.NewControlStates.Add(ObjectId, bIsValidForSending);
					}
				}

				return EBreakBehavior::Continue;
			});

		return SyncControlChange;
	}

	void FSyncControlManager::RefreshAndSendToAllClientsExcept(const FGuid& SkippedClient)
	{
		Getters.ForEachReplicationClient([this, &SkippedClient](const FGuid& ClientId)
		{
			if (SkippedClient != ClientId)
			{
				RefreshAndSendSyncControl(ClientId);
			}

			return EBreakBehavior::Continue;
		});
	}
	
	bool FSyncControlManager::IsAnyoneInterestedIn(const FConcertReplicatedObjectId& Object) const
	{
		bool bFoundClient = false;
		Getters.ForEachReplicationClient([this, &Object, &bFoundClient](const FGuid& ListenerId)
		{
			bFoundClient |= IsClientInterestedIn(Object, ListenerId);
			return bFoundClient ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
		return bFoundClient;
	}

	bool FSyncControlManager::IsClientInterestedIn(const FConcertReplicatedObjectId& Object, const FGuid& Client) const
	{
		// For now, we'll just prevent sending if there is no other client.
		// TODO: In the future, we should consider whether Client in the same world as Object.SenderEndpointId
		const bool bIsSameAsSender = Client == Object.SenderEndpointId;
		return !bIsSameAsSender;
	}
}

