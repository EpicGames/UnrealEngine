// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Muting/MuteManager.h"
#include "Replication/Data/ObjectIds.h"
#include "Replication/Messages/SyncControl.h"

#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ConcertSyncServer::Replication
{
	class FMuteManager;
}

class IConcertServerSession;
enum class EConcertClientStatus : uint8;
struct FConcertSessionClientInfo;

namespace UE::ConcertSyncServer::Replication
{
	class IRegistrationEnumerator;
	class FAuthorityManager;
	
	/**
	 * Decides whether clients should be replicating.
	 * Clients may replicate when they have authority and there are other clients listening for that data.
	 *
	 * For now, sync control just checks whether there is another client in the session.
	 * TODO: In the future, we should consider whether there is another client in the same world, as well.
	 */
	class FSyncControlManager : public FNoncopyable
	{
	public:
		
		FSyncControlManager(
			IConcertServerSession& Session UE_LIFETIMEBOUND,
			FAuthorityManager& AuthorityManager UE_LIFETIMEBOUND,
			FMuteManager& MuteManager UE_LIFETIMEBOUND,
			const IRegistrationEnumerator& Getters UE_LIFETIMEBOUND
			);
		~FSyncControlManager();

		/** @return Whether this Object is allowed to processed. */
		bool HasSyncControl(const FConcertReplicatedObjectId& Object) const;
		/** @return If the client has any controlled objects, returns a valid pointer to the set of objects they own. */
		const TSet<FConcertObjectInStreamID>* GetClientControlledObjects(const FGuid& ClientId) const;

		/** Called by FConcertServerReplicationManager when client completes replication handshake. */
		FConcertReplication_ChangeSyncControl OnGenerateSyncControlForClientJoin(const FGuid& ClientId);
		
		/** Called by FConcertServerReplicationManager when client completes leaves replication. */
		void OnPostClientLeft(const FGuid& ClientId) { HandleClientLeave(ClientId); }

	private:

		/** Used to send sync control messages to clients, which notifies them to start / stop replicating. */
		IConcertServerSession& Session;
		/** Used to detect whether a client has authority. */
		FAuthorityManager& AuthorityManager;
		/** Used to detect whether objects are globally muted. */
		FMuteManager& MuteManager;

		/** Callbacks for retrieving more info about client replication registration. */
		const IRegistrationEnumerator& Getters;

		struct FClientData
		{
			/** Contains all objects that are allowed to be replicated for this client. */
			TSet<FConcertObjectInStreamID> ObjectsWithSyncControl;
		};
		/** Maps client ID to client sync control data. */
		TMap<FGuid, FClientData> PerClientData;

		/** Generates new sync control for client that is explicitly changing their authority. */
		FConcertReplication_ChangeSyncControl OnGenerateSyncControlForAuthorityResponse(const FGuid& ClientId);

		/**
		 * Called when a client has caused mute state to change (e.g. due to removing object from stream, or explicitly muting it).
		 * Called via delegate by FMuteManager.
		 */
		void OnRefreshSyncControlForIndirectMuteChange(const FGuid& ClientId)
		{
			// Do not send any update to this client (because they can infer the change themselves) but do update the sync control state.
			RefreshClientSyncControl(ClientId);
			// The other clients need to be notified in case an object was added which is now implicitly muted.
			RefreshAndSendToAllClientsExcept(ClientId);
		}
		/** 
		 * Updates sync control for all clients, sends an update to all clients but ClientId, and returns the sync control to embed into the mute response.
		 * Called via delegate by FMuteManager.
		 */
		FConcertReplication_ChangeSyncControl OnRefreshSyncControlForClientMuteChange(const FGuid& ClientId);
		/** Updates sync control for all clients and enumerates the sync control instead of sending it. The callback handles updating the remote client machines. */
		void OnRefreshSyncControlAndEnumerateWithoutSending(const FMuteManager::FOnSyncControlChange& OnSyncControlChange);
		
		/** Cleans up the associated client data and updates sync control for all other clients. */
		void HandleClientLeave(const FGuid& LeftClientId);

		/** Compares the client's current sync control against the new sync control it should have and returns the delta to be sent to the client. */
		FConcertReplication_ChangeSyncControl RefreshClientSyncControl(
			const FGuid& ClientId, 
			TFunctionRef<bool(const FConcertObjectInStreamID& Object, bool bNewValue)> ShouldSkipInMessage = [](auto&, auto){ return false; }
			);

		/** Except for SkippedClient, checks all clients' sync control has changed and conditionally updates the remote endpoints. */
		void RefreshAndSendToAllClientsExcept(const FGuid& SkippedClient);
		/** Checks whether client sync control has changed and conditionally updates the remote endpoint. */
		void RefreshAndSendSyncControl(const FGuid& ClientId);

		/** @return Whether any client wants to receive Object. */
		bool IsAnyoneInterestedIn(const FConcertReplicatedObjectId& Object) const;
		/** @return Whether Client is interested in receiving Object. */
		bool IsClientInterestedIn(const FConcertReplicatedObjectId& Object, const FGuid& Client) const;
	};
}

