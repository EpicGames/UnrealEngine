// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroundTruthOverride.h"

#include "Replication/AuthorityManager.h"
#include "Replication/ConcertReplicationClient.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Data/ReplicationStreamArray.h"

namespace UE::ConcertSyncServer::Replication
{
	void FGroundTruthOverride::ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)> Callback) const
	{
		if (const FConcertReplicationStreamArray* StreamsOverride = StreamOverrides.Find(ClientEndpointId))
		{
			for (const FConcertReplicationStream& Stream : StreamsOverride->Streams)
			{
				if (Callback(Stream.BaseDescription.Identifier, Stream.BaseDescription.ReplicationMap) == EBreakBehavior::Break)
				{
					break;
				}
			}
		}
		else
		{
			NoOverrideStreamFallback.ForEachStream(ClientEndpointId, [&Callback](const FConcertReplicationStream& Stream)
			{
				return Callback(Stream.BaseDescription.Identifier, Stream.BaseDescription.ReplicationMap);
			});
		}
	}

	void FGroundTruthOverride::ForEachClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const
	{
		TSet<FGuid> AlreadyListed;

		// First list every client that is actually registered on the server...
		EBreakBehavior BreakBehavior = EBreakBehavior::Continue;
		NoOverrideStreamFallback.ForEachReplicationClient([&Callback, &AlreadyListed, &BreakBehavior](const FGuid& EndpointId)
		{
			BreakBehavior = Callback(EndpointId);
			AlreadyListed.Add(EndpointId);
			return BreakBehavior;
		});

		if (BreakBehavior == EBreakBehavior::Break)
		{
			return;
		}

		// ... and then any further clients that are injected by the overrides (e.g. clients that have since disconnected).
		for (const TPair<FGuid, FConcertReplicationStreamArray>& Overrides : StreamOverrides)
		{
			const FGuid& OverrideStream = Overrides.Key;
			if (!AlreadyListed.Contains(OverrideStream)
				&& Callback(OverrideStream) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	bool FGroundTruthOverride::HasAuthority(const FGuid& ClientId, const FGuid& StreamId, const FSoftObjectPath& ObjectPath) const
	{
		const FConcertReplicatedObjectId ObjectId {{ StreamId, ObjectPath }, ClientId };
		const FConcertObjectInStreamArray* AuthorityOverride = AuthorityOverrides.Find(ClientId);
		return AuthorityOverride
			? AuthorityOverride->Objects.Contains(ObjectId)
			: NoOverrideAuthorityFallback.HasAuthorityToChange(ObjectId);
	}
}
