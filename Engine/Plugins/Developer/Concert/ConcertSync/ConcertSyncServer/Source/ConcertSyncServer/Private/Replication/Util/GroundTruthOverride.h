// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/AuthorityConflictSharedUtils.h"
#include "Replication/Misc/IReplicationGroundTruth.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Misc/Guid.h"
#include "Templates/UnrealTemplate.h"

struct FConcertReplicationStreamArray;
struct FConcertObjectInStreamArray;

namespace UE::ConcertSyncServer::Replication
{
	class FAuthorityManager;
	class FConcertReplicationClient;
	class IRegistrationEnumerator;
	
	/**
	 * Pretends that the ground truth is the client overrides it was given.
	 * If a setting is not overriden, then it defaults back to the server state.
	 * 
	 * StreamOverrides can contain clients that are not actually connected ("injection").
	 * This is useful if you want to validate i.e. that no authority conflicts happen if the injected clients were present. 
	 */
	class FGroundTruthOverride
		: public ConcertSyncCore::Replication::IReplicationGroundTruth
		, public FNoncopyable
	{
	public:
		
		FGroundTruthOverride(
			const TMap<FGuid, FConcertReplicationStreamArray>& StreamOverrides UE_LIFETIMEBOUND,
			const TMap<FGuid, FConcertObjectInStreamArray>& AuthorityOverrides UE_LIFETIMEBOUND,
			const IRegistrationEnumerator& NoOverrideStreamFallback UE_LIFETIMEBOUND,
			const FAuthorityManager& NoOverrideAuthorityFallback UE_LIFETIMEBOUND
			)
			: StreamOverrides(StreamOverrides)
			, AuthorityOverrides(AuthorityOverrides)
			, NoOverrideStreamFallback(NoOverrideStreamFallback)
			, NoOverrideAuthorityFallback(NoOverrideAuthorityFallback)
		{}

		//~ Begin IReplicationGroundTruth Interface
		virtual void ForEachStream(const FGuid& ClientEndpointId, TFunctionRef<EBreakBehavior(const FGuid& StreamId, const FConcertObjectReplicationMap& ReplicationMap)> Callback) const override;
		virtual void ForEachClient(TFunctionRef<EBreakBehavior(const FGuid& ClientEndpointId)> Callback) const override;
		virtual bool HasAuthority(const FGuid& ClientId, const FGuid& StreamId, const FSoftObjectPath& ObjectPath) const override;
		//~ Begin IReplicationGroundTruth Interface

	private:

		const TMap<FGuid, FConcertReplicationStreamArray>& StreamOverrides;
		const TMap<FGuid, FConcertObjectInStreamArray>& AuthorityOverrides;

		/** Gives us the stream content when no override was specified. */
		const IRegistrationEnumerator& NoOverrideStreamFallback;
		/** Gives us the authority when no override was specified. */
		const FAuthorityManager& NoOverrideAuthorityFallback;
	};
}


