// Copyright Epic Games, Inc. All Rights Reserved.

#include "JoinRequestValidation.h"

#include "Replication/Data/ObjectReplicationMap.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Messages/Handshake.h"

namespace UE::ConcertSyncServer::Replication
{
	static TTuple<EJoinReplicationErrorCode, FString> CheckInputConstraints(const FConcertReplication_Join_Request& Request)
	{
		/* Duplicate objects and properties in a request might not actually cause any trouble.
		 * However, we will straight out reject any input that is not exactly how we expect it.
		 * This serves us to
		 *	1. reason about our pre/post conditions
		 *	2. since other code can assume said conditions, it is a common security mechanic to reject input in case somebody sends us specifically crafted input
		 */
		
		TSet<FGuid> UniqueStreamsFromRequestor;
		for (const FConcertReplicationStream& Stream : Request.Streams)
		{
			const FGuid& StreamId = Stream.BaseDescription.Identifier;

			// Request contained stream ID twice?
			if (UniqueStreamsFromRequestor.Contains(StreamId))
			{
				return { EJoinReplicationErrorCode::DuplicateStreamId,  FString::Printf(TEXT("Duplicate stream ID %s in request."), *StreamId.ToString()) };
			}
			UniqueStreamsFromRequestor.Add(Stream.BaseDescription.Identifier);

			// Make sure the properties are valid
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& ReplicatedObjectInfo : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				const FSoftObjectPath& ObjectPath = ReplicatedObjectInfo.Key;
				const FConcertReplicatedObjectInfo& ObjectInfo = ReplicatedObjectInfo.Value;
				
				// Validate class - we cannot validate at this point whether the class really exists.
				if (!ObjectInfo.ClassPath.IsValid())
				{
					const FString ErrorMessage = FString::Printf(
						TEXT("Stream %s has null class for object %s"),
						*Stream.BaseDescription.Identifier.ToString(),
						*ObjectPath.ToString()
						);
					return { EJoinReplicationErrorCode::InvalidClass, ErrorMessage };
				}
			}
		}

		return { EJoinReplicationErrorCode::Success, TEXT("") };
	}
	
	TTuple<EJoinReplicationErrorCode, FString, TArray<FConcertReplicationStream>> ValidateRequest(const FConcertReplication_Join_Request& Request)
	{
		if (auto[ErrorCode, ErrorMessage] = CheckInputConstraints(Request)
			; ErrorCode != EJoinReplicationErrorCode::Success)
		{
			return { ErrorCode, ErrorMessage, TArray<FConcertReplicationStream>{} };
		}

		return { EJoinReplicationErrorCode::Success, TEXT(""), Request.Streams };
	}
}
