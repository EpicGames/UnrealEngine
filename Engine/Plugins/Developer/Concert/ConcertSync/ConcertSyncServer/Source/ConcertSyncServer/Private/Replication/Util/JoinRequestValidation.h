// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/Tuple.h"

enum class EJoinReplicationErrorCode : uint8;
struct FConcertReplication_Join_Request;
struct FConcertReplicationStream;

namespace UE::ConcertSyncServer::Replication
{
	/**
	 * Attempts to unpack the request and validates it.
	 * Constraints:
	 *  - All stream IDs must be unique (within request and clients)
	 *  - No overlapping properties (however, multiple streams can replicate the same object)
	 *  - The stream descriptions must unpack successfully (i.e. the attributes must be known)
	 * 
	 * @param Request The request to validate
	 * @param Clients Clients that are already connected
	 * @param GetClientNameFunc Callback for getting a client's display name for error reporting purposes.
	 * 
	 * @return The error code for the request and the unpacked stream descriptions
	 */
	TTuple<EJoinReplicationErrorCode, FString, TArray<FConcertReplicationStream>> ValidateRequest(const FConcertReplication_Join_Request& Request);
}
