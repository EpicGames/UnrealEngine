// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertLogGlobal.h"
#include "JsonObjectConverter.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "IConcertSession.h"

#include <type_traits>

namespace UE::ConcertSyncServer
{
	enum class EClientNameFlags
	{
		None,
		IncludeName = 1 << 0,
		IncludeEndpointId = 1 << 1
	};
	ENUM_CLASS_FLAGS(EClientNameFlags);
	
	inline TOptional<FString> GetClientName(IConcertSession& Session, const FGuid& EndpointId, EClientNameFlags Flags = EClientNameFlags::IncludeName | EClientNameFlags::IncludeEndpointId)
	{
		FConcertSessionClientInfo Info;
		if (Session.FindSessionClient(EndpointId, Info))
		{
			if (EnumHasAllFlags(Flags, EClientNameFlags::IncludeName | EClientNameFlags::IncludeEndpointId))
			{
				return FString::Printf(TEXT("%s(%s)"), *Info.ClientInfo.DisplayName, *EndpointId.ToString());
			}
			if (EnumHasAnyFlags(Flags, EClientNameFlags::IncludeName))
			{
				return Info.ClientInfo.DisplayName;
			}
			if (EnumHasAnyFlags(Flags, EClientNameFlags::IncludeEndpointId))
			{
				return EndpointId.ToString();
			}
		}
		return {};
	}
	
	template<typename TMessage, typename TGetClientName> requires std::is_invocable_r_v<TOptional<FString>, TGetClientName>
	void LogNetworkMessage(const TMessage& Message, TGetClientName GetClientName = [](){ return TOptional<FString>{}; })
	{
		FString JsonString;
		FJsonObjectConverter::UStructToJsonObjectString(TMessage::StaticStruct(), &Message, JsonString, 0, 0);
		
		if (const TOptional<FString> ClientName = GetClientName())
		{
			UE_LOGF(LogConcert, Log, "%ls from %ls\n%ls", *TMessage::StaticStruct()->GetName(), **ClientName, *JsonString);
		}
		else
		{
			UE_LOGF(LogConcert, Log, "%ls\n%ls", *TMessage::StaticStruct()->GetName(), *JsonString);
		}
	}
	
	template<typename TMessage, typename TGetClientName> requires std::is_invocable_r_v<TOptional<FString>, TGetClientName>
	void LogNetworkMessage(const TAutoConsoleVariable<bool>& ShouldLog, const TMessage& Message, TGetClientName GetClientName = [](){ return TOptional<FString>{}; })
	{
		if (ShouldLog.GetValueOnAnyThread())
		{
			LogNetworkMessage(Message, GetClientName);
		}
	}
}
