// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationCVars.h"

namespace UE::ConcertSyncServer::Replication
{
TAutoConsoleVariable<bool> CVarLogStreamRequestsAndResponsesOnServer(
	TEXT("Concert.Replication.LogStreamRequestsAndResponsesOnServer"),
	false,
	TEXT("Whether to log changes to streams.")
	);
	
TAutoConsoleVariable<bool> CVarLogAuthorityRequestsAndResponsesOnServer(
	TEXT("Concert.Replication.LogAuthorityRequestsAndResponsesOnServer"),
	false,
	TEXT("Whether to log changes to authority.")
	);
}