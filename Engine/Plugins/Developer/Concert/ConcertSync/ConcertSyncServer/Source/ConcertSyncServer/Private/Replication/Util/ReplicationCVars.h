// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

namespace UE::ConcertSyncServer::Replication
{
/** Whether to log requests and responses to streams on the server. */
extern TAutoConsoleVariable<bool> CVarLogStreamRequestsAndResponsesOnServer;
	
/** Whether to log requests and responses to authority on the server. */
extern TAutoConsoleVariable<bool> CVarLogAuthorityRequestsAndResponsesOnServer;
}
