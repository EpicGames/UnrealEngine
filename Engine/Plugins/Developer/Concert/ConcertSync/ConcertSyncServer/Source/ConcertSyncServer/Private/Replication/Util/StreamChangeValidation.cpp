// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamChangeValidation.h"

#include "ConcertLogGlobal.h"
#include "Replication/AuthorityManager.h"
#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/ConcertReplicationClient.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Messages/ChangeStream.h"

namespace UE::ConcertSyncServer::Replication
{
	namespace Private
	{
		static const FConcertReplicationStream* FindExistingStream(const TConstArrayView<FConcertReplicationStream>& Streams, const FGuid& StreamId)
		{
			return Streams.FindByPredicate([&StreamId](const FConcertReplicationStream& Description)
			{
				return Description.BaseDescription.Identifier == StreamId;
			});
		}
		
		/** Validates that ObjectsToPut writes only to pre-existing streams. */
		static void ValidatePutObjectsRequestSemantics(const TConstArrayView<FConcertReplicationStream>& Streams, const FConcertReplication_ChangeStream_Request& Request, FConcertReplication_ChangeStream_Response& OutResponse)
		{
			const auto PutObjectHasEnoughData = [](const FConcertReplicationStream* ExistingStream, const FSoftObjectPath& ChangedObjectPath, const FConcertReplication_ChangeStream_PutObject& PutObject)
			{
				const bool bHasProperties = !PutObject.Properties.ReplicatedProperties.IsEmpty();
				const bool bHasClassPath = !PutObject.ClassPath.IsNull();
				const bool bIsEditingExistingObjectDefinition = ExistingStream->BaseDescription.ReplicationMap.ReplicatedObjects.Contains(ChangedObjectPath);
				const bool bHasEnoughData = (!bIsEditingExistingObjectDefinition && bHasProperties && bHasClassPath)
					|| (bIsEditingExistingObjectDefinition && bHasProperties)
					|| (bIsEditingExistingObjectDefinition && bHasClassPath);
				return bHasEnoughData;
			};
			
			for (const TPair<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& Change : Request.ObjectsToPut)
			{
				const FConcertObjectInStreamID ChangedObject = Change.Key;
				
				const FGuid& StreamToModify = ChangedObject.StreamId;
				const FConcertReplicationStream* ExistingStream = FindExistingStream(Streams, StreamToModify);
				const bool bStreamExists = ExistingStream != nullptr;
				if (bStreamExists)
				{
					// FConcertReplicatedObjectInfo must have non-empty values for its members. Hence we must check that
					// 1. if creating a new entry, both fields are given
					// 2. if writing to pre-existing entry, at least one field is given
					const FConcertReplication_ChangeStream_PutObject& PutObject = Change.Value; 
					if (!PutObjectHasEnoughData(ExistingStream, ChangedObject.Object, PutObject))
					{
						OutResponse.ObjectsToPutSemanticErrors.Add(ChangedObject, EConcertPutObjectErrorCode::MissingData);
					}
				}
				else
				{
					// ObjectsToPut can only write to pre-existing streams
					UE_LOGF(LogConcert, Log, "Semantic error: unknown stream %ls", *StreamToModify.ToString(EGuidFormats::Short));
					OutResponse.ObjectsToPutSemanticErrors.Add(ChangedObject, EConcertPutObjectErrorCode::UnresolvedStream);
				}
			}
		}

		/** Checks that StreamsToAdd do not conflict with pre-existing ones and that all IDs in the request are also unique. */
		static void ValidateAddedStreamsAreValid(const TConstArrayView<FConcertReplicationStream>& Streams, const FConcertReplication_ChangeStream_Request& Request, FConcertReplication_ChangeStream_Response& OutResponse)
		{
			TSet<FGuid> DuplicateEntryDetection;
			for (const FConcertReplicationStream& NewStream : Request.StreamsToAdd)
			{
				// StreamsToAdd is invalid if there is already a stream with the same ID registered ...
				const FGuid& NewStreamId = NewStream.BaseDescription.Identifier;
				const bool bIdAlreadyExists = FindExistingStream(Streams, NewStreamId) != nullptr;
				const bool bIsStreamRemoved = Request.StreamsToRemove.Contains(NewStreamId);

				// ... or StreamsToAdd contains the same ID multiple times
				bool bIsDuplicateEntry = false;
				DuplicateEntryDetection.FindOrAdd(NewStreamId, &bIsDuplicateEntry);
				
				if ((bIdAlreadyExists && !bIsStreamRemoved)
					|| bIsDuplicateEntry
					|| NewStream.BaseDescription.ReplicationMap.IsEmpty())
				{
					UE_LOGF(LogConcert, Log, "Failed to create stream %ls", *NewStreamId.ToString(EGuidFormats::Short));
					OutResponse.FailedStreamCreation.Add(NewStreamId);
				}
			}
		}

		/** If the requesting client has authority over a changed object, checks that no other client has authority over the properties being added. */
		static void LookForAuthorityConflicts(const FGuid& ClientEndpointId, const FConcertReplication_ChangeStream_Request& Request, const FAuthorityManager& AuthorityManager, FConcertReplication_ChangeStream_Response& OutResponse)
		{
			for (const TPair<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& PutObjectPair : Request.ObjectsToPut)
			{
				// No conflict possible if requesting client does not have authority over the changed object
				const FConcertReplicatedObjectId ReplicatedObjectInfo { { PutObjectPair.Key }, ClientEndpointId };
				if (!AuthorityManager.HasAuthorityToChange(ReplicatedObjectInfo))
				{
					continue;
				}

				// Requester not changing properties (must be changing ClassPath)? Also no conflict possible.
				const FConcertPropertySelection& PropertySelection = PutObjectPair.Value.Properties;
				const bool bIsEditingProperties = !PropertySelection.ReplicatedProperties.IsEmpty();
				if (!bIsEditingProperties)
				{
					continue;
				}

				// Simply check whether any other client is already sending any of the requested properties.
				AuthorityManager.EnumerateAuthorityConflicts(ReplicatedObjectInfo, &PropertySelection,
					[&OutResponse, &ReplicatedObjectInfo](const FGuid& ClientId, const FGuid& StreamId, const FConcertPropertyChain& Property)
					{
						const FConcertReplicatedObjectId ConflictingObject = { { StreamId, ReplicatedObjectInfo.Object }, ClientId };
						OutResponse.AuthorityConflicts.Add(ReplicatedObjectInfo, ConflictingObject);
						
						UE_LOGF(LogConcert, Log, "Authority conflict with client %ls for stream %ls for property %ls", *ClientId.ToString(EGuidFormats::Short), *StreamId.ToString(EGuidFormats::Short), *Property.ToString());
						return EBreakBehavior::Continue;
					});
			}
		}
	}
	
	bool ValidateStreamChangeRequest(
		const FGuid& ClientEndpointId,
		const TConstArrayView<FConcertReplicationStream>& Streams,
		const FAuthorityManager& AuthorityManager,
		const FConcertReplication_ChangeStream_Request& Request,
		FConcertReplication_ChangeStream_Response& OutResponse
		)
	{
		OutResponse.ErrorCode = EReplicationResponseErrorCode::Handled;

		Private::ValidatePutObjectsRequestSemantics(Streams, Request, OutResponse);
		Private::ValidateAddedStreamsAreValid(Streams, Request, OutResponse);
		Private::LookForAuthorityConflicts(ClientEndpointId, Request, AuthorityManager, OutResponse);
		ConcertSyncCore::Replication::ChangeStreamUtils::ValidateFrequencyChanges(Request, Streams, &OutResponse.FrequencyErrors);
			
		return OutResponse.IsSuccess();
	}

	bool ValidateStreamChangeRequest(
		const FGuid& ClientEndpointId,
		const TConstArrayView<FConcertReplicationStream>& Streams,
		const FAuthorityManager& AuthorityManager,
		const FConcertReplication_ChangeStream_Request& Request
		)
	{
		FConcertReplication_ChangeStream_Response Dummy;
		return ValidateStreamChangeRequest(ClientEndpointId, Streams, AuthorityManager, Request, Dummy);
	}
}
