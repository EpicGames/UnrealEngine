// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"

struct FConcertReplicationStream;
struct FConcertReplication_ChangeStream_Response;
struct FConcertReplication_ChangeStream_Request;

namespace UE::ConcertSyncServer::Replication
{
	class FAuthorityManager;
	class FConcertReplicationClient;

	/**
	 * Checks whether the stream change is valid to make and puts all errors into OutResponse.
	 * @return Whether the request is valid.
	 */
	bool ValidateStreamChangeRequest(
		const FGuid& ClientEndpointId,
		const TConstArrayView<FConcertReplicationStream>& Streams,
		const FAuthorityManager& AuthorityManager,
		const FConcertReplication_ChangeStream_Request& Request,
		FConcertReplication_ChangeStream_Response& OutResponse
		);
	
	/** @return Whether the request is valid. */
	bool ValidateStreamChangeRequest(
		const FGuid& ClientEndpointId,
		const TConstArrayView<FConcertReplicationStream>& Streams,
		const FAuthorityManager& AuthorityManager,
		const FConcertReplication_ChangeStream_Request& Request
		);
}


