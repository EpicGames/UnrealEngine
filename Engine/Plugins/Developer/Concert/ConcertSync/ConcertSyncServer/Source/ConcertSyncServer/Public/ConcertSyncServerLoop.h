// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertSyncServerLoopInitArgs.h"

/**
 * Utility method to shutdown the ConcertSyncServer.
 */
void ShutdownConcertSyncServer(const FString& ServiceFriendlyName);

/**
 * Blocking main loop for running a Concert Sync server application.
 */
int32 ConcertSyncServerLoop(const TCHAR* CommandLine, const FConcertSyncServerLoopInitArgs& InitArgs);

#include "ConcertSyncServerLoop.inl"
