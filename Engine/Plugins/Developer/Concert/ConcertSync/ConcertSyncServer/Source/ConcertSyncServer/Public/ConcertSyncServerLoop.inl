// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncServerLoop.h"
#include "ConcertSettings.h"
#include "ConcertServerSettings.h"
#include "IConcertServer.h"
#include "IConcertSyncServer.h"
#include "IConcertSyncServerModule.h"

#include "CoreGlobals.h"
#include "LaunchEngineLoop.h"
#include "Containers/Ticker.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceConsole.h"
#include "Interfaces/IPluginManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Stats/StatsSystem.h"

#include "Runtime/Launch/Resources/Version.h"

DEFINE_LOG_CATEGORY_STATIC(LogSyncServer, Log, All);

void ShutdownConcertSyncServer(const FString& ServiceFriendlyName)
{
	UE_LOGF(LogSyncServer, Display, "%ls Shutdown", *ServiceFriendlyName);

	// Allow the game thread to finish processing any latent tasks.
	// They will be relying on what we are going to shutdown...
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

	FEngineLoop::AppPreExit();

	// Unloading Modules isn't handled by AppExit
	FModuleManager::Get().UnloadModulesAtShutdown();

	// On Mac AppExit is passed as delegate to the OS, which will call it implicitly
#if !PLATFORM_MAC
	FEngineLoop::AppExit();
#endif
}

int32 ConcertSyncServerLoop(const TCHAR* CommandLine, const FConcertSyncServerLoopInitArgs& InitArgs)
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	
	// Validate the init settings
	checkf(InitArgs.IdealFramerate > 0, TEXT("IdealFramerate must be greater than zero!"));
	checkf(InitArgs.SessionFlags != EConcertSyncSessionFlags::None, TEXT("SessionFlags cannot be None!"));
	checkf(!InitArgs.ServiceRole.IsEmpty(), TEXT("ServiceRole cannot be empty!"));
	checkf(!InitArgs.ServiceFriendlyName.IsEmpty(), TEXT("ServiceFriendlyName cannot be empty!"));

	// start up the main loop, adding some extra command line arguments:
	//	-Messaging enables MessageBus transports
	const int32 Result = GEngineLoop.PreInit(*FString::Printf(TEXT("%s %s"), CommandLine, TEXT(" -Messaging")));
	check(GConfig && GConfig->IsReadyForUse());

	if (Result < 0)
	{
		UE_LOGF(LogSyncServer, Error, "EngineLoop PreInit failed!");

		ShutdownConcertSyncServer(InitArgs.ServiceFriendlyName);

		return Result;
	}

	if (InitArgs.bShowConsole)
	{
		GLogConsole->Show(true);
		UE_LOGF(LogSyncServer, Display, "Launch : %ls", CommandLine);
	}

	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	// Load internal Concert plugins in the pre-default phase
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault);

	// Load Concert Sync plugins in default phase
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::Default);

	// Load Concert Sync plugins in post default phase.
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostDefault);

	// Install graceful termination handler, this handles graceful CTRL+C shutdown,
	// but not CTRL+CLOSE, which will potentially still exit process before the main thread exits.
	// Double CTRL+C signal will also cause process to terminate.
	FPlatformMisc::SetGracefulTerminationHandler();

	if (!IConcertSyncServerModule::IsAvailable())
	{
		UE_LOGF(LogSyncServer, Error, "ConcertSyncServer Module is needed for proper execution. Initialization failed!");

		ShutdownConcertSyncServer(InitArgs.ServiceFriendlyName);

		return -1;
	}

	const UConcertServerConfig* ServerConfig = nullptr;

	// Get the server config.
	if (InitArgs.GetServerConfigFunc)
	{
		ServerConfig = InitArgs.GetServerConfigFunc();
	}
	else
	{
		ServerConfig = IConcertSyncServerModule::Get().ParseServerSettings(FCommandLine::Get());
	}

	if (!ServerConfig)
	{
		UE_LOGF(LogSyncServer, Error, "%ls Configuration Failed!", *InitArgs.ServiceFriendlyName);
		
		ShutdownConcertSyncServer(InitArgs.ServiceFriendlyName);

		return -1;
	}


	// Give external modules to do early initialisation
	InitArgs.PreInitServerLoop.Broadcast();

	TSharedPtr<IPlugin> UdpPlugin = IPluginManager::Get().FindPlugin(TEXT("UdpMessaging"));
	TSharedPtr<IPlugin> QuicPlugin = IPluginManager::Get().FindPlugin(TEXT("QuicMessaging"));

	// Either the UdpMessaging or QuicMessaging plugin
	// should be added to the {appname}.Target.cs build file.
	const bool bUdpEnabled = (UdpPlugin && UdpPlugin->IsEnabled());
	const bool bQuicEnabled = (QuicPlugin && QuicPlugin->IsEnabled());

	if (!bUdpEnabled && !bQuicEnabled)
	{
		UE_LOGF(LogSyncServer, Error, "The 'UDP Messaging' and 'QUIC Messaging' plugins are disabled. "
			"The Concert server only supports UDP/QUIC protocol.");

		ShutdownConcertSyncServer(InitArgs.ServiceFriendlyName);

		return -1;
	}

	// Setup Concert Sync to run in server mode.
	TSharedPtr<IConcertSyncServer> ConcertSyncServer = IConcertSyncServerModule::Get().CreateServer(InitArgs.ServiceRole, InitArgs.ServiceAutoArchiveSessionFilter);
	ConcertSyncServer->SetFileSharingService(InitArgs.GetFileSharingServiceFunc ? InitArgs.GetFileSharingServiceFunc() : InitArgs.FileSharingService);
	ConcertSyncServer->Startup(ServerConfig, InitArgs.SessionFlags);

	// if we have a default session, set it up properly
	if (!ServerConfig->DefaultSessionName.IsEmpty())
	{
		IConcertServerRef ConcertServer = ConcertSyncServer->GetConcertServer();
		if (!ConcertServer->GetLiveSessionIdByName(ServerConfig->DefaultSessionName).IsValid())
		{
			FConcertSessionInfo SessionInfo = ConcertServer->CreateSessionInfo();
			SessionInfo.SessionName = ServerConfig->DefaultSessionName;
			SessionInfo.Settings = ServerConfig->DefaultSessionSettings;
			SessionInfo.VersionInfos.Add(ServerConfig->DefaultVersionInfo);

			bool bSuccess = false;
			if (!ServerConfig->DefaultSessionToRestore.IsEmpty())
			{
				const FGuid ArchivedSessionId = ConcertServer->GetArchivedSessionIdByName(ServerConfig->DefaultSessionToRestore);
				if (ArchivedSessionId.IsValid())
				{
					FText FailureReason;
					bSuccess = ConcertServer->RestoreSession(ArchivedSessionId, SessionInfo, FConcertSessionFilter(), FailureReason).IsValid();
					if (!bSuccess)
					{
						UE_LOGF(LogSyncServer, Error, "%ls failed to restore archived session '%ls' due to '%ls'. Creating new session instead...", *InitArgs.ServiceFriendlyName, *ServerConfig->DefaultSessionToRestore, *FailureReason.ToString());
					}
				}
				else
				{
					UE_LOGF(LogSyncServer, Error, "%ls could not find archived session '%ls' to restore. Creating new session instead...", *InitArgs.ServiceFriendlyName, *ServerConfig->DefaultSessionToRestore);
				}
			}

			if (!bSuccess)
			{
				FText FailureReason;
				bSuccess = ConcertServer->CreateSession(SessionInfo, FailureReason).IsValid();
				if (!bSuccess)
				{
					UE_LOGF(LogSyncServer, Error, "%ls failed to create session '%ls' due to '%ls'.", *InitArgs.ServiceFriendlyName, *ServerConfig->DefaultSessionName, *FailureReason.ToString());
				}
			}
		}
	}

	UE_LOGF(LogSyncServer, Display, "%ls Initialized (Name: %ls, Version: %d.%d, Role: %ls)", *InitArgs.ServiceFriendlyName, *ConcertSyncServer->GetConcertServer()->GetServerInfo().ServerName, ENGINE_MAJOR_VERSION, ENGINE_MINOR_VERSION, *ConcertSyncServer->GetConcertServer()->GetRole());
	InitArgs.PostInitServerLoop.Broadcast(ConcertSyncServer.ToSharedRef());

	double LastTime = FPlatformTime::Seconds();
	const float IdealFrameTime = 1.0f / InitArgs.IdealFramerate;

	while (!IsEngineExitRequested())
	{
		const double CurrentTime = FPlatformTime::Seconds();
		const double DeltaTime = CurrentTime - LastTime;

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		// Pump & Tick objects
		InitArgs.TickPostGameThread.Broadcast(DeltaTime);
		FTSTicker::GetCoreTicker().Tick(DeltaTime);

		GFrameCounter++;
		UE::Stats::FStats::AdvanceFrame(false);
		GLog->FlushThreadedLogs();

		// Run garbage collection for the UObjects for the rest of the frame or at least to 2 ms
		IncrementalPurgeGarbage(true, FMath::Max<float>(0.002f, IdealFrameTime - (FPlatformTime::Seconds() - LastTime)));

#if PLATFORM_MAC
		// Pumps the message from the main loop. (On Mac, we have a full application to get a proper console window to output logs)
		FPlatformApplicationMisc::PumpMessages(true);
#endif

		// Throttle main thread main fps by sleeping if we still have time
		FPlatformProcess::Sleep(FMath::Max<float>(0.0f, IdealFrameTime - (FPlatformTime::Seconds() - LastTime)));

		LastTime = CurrentTime;
	}

	ConcertSyncServer->Shutdown();
	ConcertSyncServer.Reset();

	ShutdownConcertSyncServer(InitArgs.ServiceFriendlyName);

	return Result;
}
