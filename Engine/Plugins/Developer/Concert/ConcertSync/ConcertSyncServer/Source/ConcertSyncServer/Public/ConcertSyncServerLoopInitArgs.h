// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"
#include "ConcertSyncSessionFlags.h"

class UConcertServerConfig;
class IConcertFileSharingService;
class IConcertSyncServer;

struct FConcertSyncServerLoopInitArgs
{
	DECLARE_MULTICAST_DELEGATE(FMultiUserServerLoopInitEvent);
	DECLARE_MULTICAST_DELEGATE_OneParam(FMultiUserServerInitServerEvent, TSharedRef<IConcertSyncServer>);
	DECLARE_MULTICAST_DELEGATE_OneParam(FMultiUserServerLoopTickEvent, double /* DeltaTime */);
	
	/** Framerate that the main loop should try to Tick at */
	int32 IdealFramerate = 60;

	/** Flags controlling what features are enabled for sessions within this server */
	EConcertSyncSessionFlags SessionFlags = EConcertSyncSessionFlags::None;

	/** The role that this server will perform (eg, MultiUser, DisasterRecovery, etc) */
	FString ServiceRole;

	/** Friendly name to use for this service (when showing it to a user in log messages, etc) */
	FString ServiceFriendlyName;

	/** The session filter to apply when auto-archiving sessions on this server */
	FConcertSessionFilter ServiceAutoArchiveSessionFilter;

	/** The optional file sharing server to exchange large files. Can be null. */
	TSharedPtr<IConcertFileSharingService> FileSharingService;

	/** Function to get the file sharing service during initialization of concert server. f this is defined then it takes precedence over FileSharingService member. */
	TFunction<TSharedPtr<IConcertFileSharingService>()> GetFileSharingServiceFunc;
	
	/** Function to get the server settings object to configure the server with with, or unbound to parse the default settings */
	TFunction<const UConcertServerConfig*()> GetServerConfigFunc;

	/** Called before Multi User Server is initialized: no modules have yet been loaded. */
	FMultiUserServerLoopInitEvent PreInitServerLoop;
	
	/**
	 * Called while initilializing Multi User server loop. External services can perform custom initialization here.
	 *
	 * Called just before entering the server loop. LoadModulesForEnabledPlugins will have been called with PreDefault 
	 * and Default and the multi user session created.
	 */
	FMultiUserServerInitServerEvent PostInitServerLoop;
	
	/**
	 * Called after the game thread's tick and before FTSTicker::GetCoreTicker.
	 */
	FMultiUserServerLoopTickEvent TickPostGameThread;

	/** Whether the service should show the log console. */
	bool bShowConsole = true;
};
