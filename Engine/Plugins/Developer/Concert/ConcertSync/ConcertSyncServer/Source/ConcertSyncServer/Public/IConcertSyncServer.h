// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertModule.h"
#include "IConcertServerModule.h"
#include "ConcertSyncSessionFlags.h"
#include "Templates/NonNullPointer.h"

class FConcertSyncSessionDatabase;
class IConcertFileSharingService;
class UConcertServerConfig;

using FConcertSyncSessionDatabaseNonNullPtr = TNonNullPtr<FConcertSyncSessionDatabase>;

/**
 * Interface for a Concert Sync Server.
 */
class IConcertSyncServer
{
public:
	virtual ~IConcertSyncServer() = default;

	/** Start this Concert Sync Server using the given config */
	virtual void Startup(const UConcertServerConfig* InServerConfig, const EConcertSyncSessionFlags InSessionFlags) = 0;

	/** Stop this Concert Sync Server */
	virtual void Shutdown() = 0;

	/** Get the current server */
	virtual IConcertServerRef GetConcertServer() const = 0;

	/** Set the file sharing service, enabling the server to work with large files. The server sharing service must be compatible with the client one. */
	virtual void SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService) = 0;

	/** Gets the session data base for the given live session if it exists */
	virtual TOptional<FConcertSyncSessionDatabaseNonNullPtr> GetLiveSessionDatabase(const FGuid& SessionId) = 0;

	/** Gets the session data base for the given archived session if it exists */
	virtual TOptional<FConcertSyncSessionDatabaseNonNullPtr> GetArchivedSessionDatabase(const FGuid& SessionId) = 0;
};
