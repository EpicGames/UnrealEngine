// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertSyncServer.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IConcertSyncServer;
class UConcertServerConfig;
class IConcertFileSharingService;

struct FConcertSessionFilter;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertSyncServerCreated, TWeakPtr<IConcertSyncServer>);

/**
 * Interface for the Concert Sync Server module.
 */
class IConcertSyncServerModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IConcertSyncServerModule& Get()
	{
		static const FName ModuleName = "ConcertSyncServer";
		return FModuleManager::LoadModuleChecked<IConcertSyncServerModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "ConcertSyncServer";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Parse command line server settings and apply them
	 * @param CommandLine the application command line arguments
	 * @return The server settings, modified or not by the command line
	 */
	virtual UConcertServerConfig* ParseServerSettings(const TCHAR* CommandLine) = 0;

	/**
	 * Create a sync server that will perform a certain role (eg, MultiUser, DisasterRecovery, etc)
	 * @param InRole The role to create
	 * @param InAutoArchiveSessionFilter The session filter to apply when auto-archiving sessions
	 * @return The server
	 */
	virtual TSharedRef<IConcertSyncServer> CreateServer(const FString& InRole, const FConcertSessionFilter& InAutoArchiveSessionFilter) = 0;

	/**
	 * Creates a shared reference to the file sharing service to be used by concert.
	 *
	 * @return a non-null shared reference to the file sharing service. 
	 */
	virtual TSharedRef<IConcertFileSharingService> CreateFileSharingService(const FString& InRole) = 0;
		
	/**
	 * Callback for when a ConcertSyncServer has been created.
	 */
	virtual FOnConcertSyncServerCreated& OnServerCreated() = 0;
};
