// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::ConcertSyncServer::Replication
{
	/**
	 * Handles all communication with clients regarding replication.
	 * 
	 * Negotiates replicated properties with clients, decides who is the authority (which is the first client to register properties),
	 * and replicates received data to other clients.
	 */
	class IConcertServerReplicationManager
	{
	public:

		virtual ~IConcertServerReplicationManager() = default;
	};
}