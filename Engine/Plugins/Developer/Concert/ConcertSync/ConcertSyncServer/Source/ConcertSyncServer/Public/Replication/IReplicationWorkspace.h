// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Templates/FunctionFwd.h"

struct FConcertSyncReplicationEvent;
enum class EBreakBehavior : uint8;

struct FConcertSyncReplicationActivity;
struct FConcertSyncReplicationPayload_Mute;
struct FConcertSessionClientInfo;
struct FConcertSyncReplicationPayload_LeaveReplication;
struct FGuid;

namespace UE::ConcertSyncServer::Replication
{
	/**
	 * Interface that FConcertServerReplicationManager uses to interact with the FConcertServerWorkspace.
	 * Allows mocking in unit tests, which is the only reason it's in the public module interface.
	 */
	class IReplicationWorkspace
	{
	public:
		
		/**
		 * Creates a replication activity for the provided client.
		 * @param EndpointId The client that produced the activity
		 * @param EventData Data associated with the activity - must have ActivityType other than EConcertSyncReplicationActivityType::None.
         * @return The identifier of the produced activity. Unset if activity insertion failed.
         */
		virtual TOptional<int64> ProduceReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationEvent& EventData) = 0;
		/** Creates a replication activity for the client leaving replication. */
		TOptional<int64> ProduceClientLeaveReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationPayload_LeaveReplication& EventData);
		/** Creates a replication activity for the client (un)muting objects in the session. */
		TOptional<int64> ProduceClientMuteReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationPayload_Mute& EventData);

		/**
		 * Gets the last replication activity associated with the given client info.
		 * 
		 * As endpoint IDs change every time a client join a session, the look up is done by client display name.
		 * If multiple machines joined with the same display name, the tie is broken by also using the device name.
		 * 
		 * @param InClientInfo Info about the client for which to get the activity.
		 * @param OutActivity The activity, if present
		 * @return Whether OutActivity contains a valid result.
		 */
		virtual bool GetLastReplicationActivityByClient(const FConcertSessionClientInfo& InClientInfo, EConcertSyncReplicationActivityType ActivityType, FConcertSyncReplicationActivity& OutActivity) const = 0;
		/** Gets the last leave replication activity associated with the given client info. */
		bool GetLastLeaveReplicationActivityByClient(const FConcertSessionClientInfo& InClientInfo, FConcertSyncReplicationPayload_LeaveReplication& OutLeaveReplication) const;
		
		/** Gets the replication leave activity with ActivityId. */
		virtual bool GetReplicationEventById(const int64 ActivityId, FConcertSyncReplicationEvent& OutEvent) const = 0;
		/** Gets the replication leave activity with ActivityId. */
		bool GetLeaveReplicationEventById(const int64 ActivityId, FConcertSyncReplicationPayload_LeaveReplication& OutLeaveReplication) const;

		/** Enumerates all activities. */
		virtual void EnumerateReplicationActivities(TFunctionRef<EBreakBehavior(const FConcertSyncReplicationActivity& Activity)> Callback) const = 0;
		/** Enumerates all mute activities. */
		void EnumerateMuteActivities(TFunctionRef<EBreakBehavior(const FConcertSyncReplicationActivity& Activity)> Callback) const;

		virtual ~IReplicationWorkspace() = default;
	};


	inline TOptional<int64> IReplicationWorkspace::ProduceClientLeaveReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationPayload_LeaveReplication& EventData)
	{
		return ProduceReplicationActivity(EndpointId, FConcertSyncReplicationEvent(EventData));
	}

	inline TOptional<int64> IReplicationWorkspace::ProduceClientMuteReplicationActivity(const FGuid& EndpointId, const FConcertSyncReplicationPayload_Mute& EventData)
	{
		return ProduceReplicationActivity(EndpointId, FConcertSyncReplicationEvent(EventData));
	}

	inline bool IReplicationWorkspace::GetLastLeaveReplicationActivityByClient(const FConcertSessionClientInfo& InClientInfo, FConcertSyncReplicationPayload_LeaveReplication& OutLeaveReplication) const
	{
		FConcertSyncReplicationActivity Activity;
		return GetLastReplicationActivityByClient(InClientInfo, EConcertSyncReplicationActivityType::LeaveReplication, Activity)
			&& ensureMsgf(Activity.EventData.ActivityType == EConcertSyncReplicationActivityType::LeaveReplication, TEXT("Caller expected ActivityId %lld to be a LeaveReplication event"), Activity.ActivityId)
			&& Activity.EventData.GetPayload(OutLeaveReplication);
	}

	inline bool IReplicationWorkspace::GetLeaveReplicationEventById(const int64 ActivityId, FConcertSyncReplicationPayload_LeaveReplication& OutLeaveReplication) const
	{
		FConcertSyncReplicationEvent Event;
		return GetReplicationEventById(ActivityId, Event)
			&& ensureMsgf(Event.ActivityType == EConcertSyncReplicationActivityType::LeaveReplication, TEXT("Caller expected ActivityId %lld to be a LeaveReplication event"), ActivityId)
			&& Event.GetPayload(OutLeaveReplication);
	}

	inline void IReplicationWorkspace::EnumerateMuteActivities(TFunctionRef<EBreakBehavior(const FConcertSyncReplicationActivity& Activity)> Callback) const
	{
		return EnumerateReplicationActivities([&Callback](const FConcertSyncReplicationActivity& Activity)
		{
			return Activity.EventData.ActivityType == EConcertSyncReplicationActivityType::Mute ? Callback(Activity) : EBreakBehavior::Continue;
		});
	}
}
